// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraitCore/TraitBinding.h"
#include "TraitCore/TraitSharedData.h"

#include "CachePoseTraitData.generated.h"

/** A trait that caches the top keyframe of the keyframe stack
 * TODO: Not implemented yet, at the moment it will simply pop the top keyframe and throw it away*/
USTRUCT(meta = (DisplayName = "Cache Pose", ShowTooltip=true, Hidden))
struct FUAFCachePoseTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()
};


