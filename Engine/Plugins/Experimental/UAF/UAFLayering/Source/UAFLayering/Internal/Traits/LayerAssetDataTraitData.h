// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "TraitCore/TraitBinding.h"
#include "TraitCore/TraitSharedData.h"
#include "UAF/UAFAssetData.h"

#include "LayerAssetDataTraitData.generated.h"


/** A trait that creates an UAF Graph at runtime based on the given asset in the layering setup and pushes it onto a blend stack*/
USTRUCT(meta = (DisplayName = "Layer Data Provider", ShowTooltip=true, Hidden))
struct FUAFLayerAssetDataTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category = "Layer|Content")
	TInstancedStruct<FUAFGraphFactoryAsset> GraphAssetHandle; 
	
	// Latent pin support boilerplate
	#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
	GeneratorMacro(GraphAssetHandle)

	GENERATE_TRAIT_LATENT_PROPERTIES(FUAFLayerAssetDataTraitSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
	#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
	
};
