// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AlphaBlend.h"
#include "UAFLayeringTypes.h"
#include "TraitCore/TraitBinding.h"
#include "TraitCore/TraitSharedData.h"

#include "LayerDataProviderTraitData.generated.h"

class UCurveFloat;

USTRUCT(BlueprintType)
struct FUAFLayerProperties
{
	GENERATED_BODY()

public:
	void AddEvent(const UE::UAF::Layering::FLayerStack_LayerEvent& Event);
	void ProcessEvents();
	
public:
	// TODO: this should be a binding 
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Layering")
	bool bLayerEnabled = true;
		
	// TODO: this should be a binding 
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Layering")
	float DesiredLayerWeight = 1.0f;
		
	// TODO: this should be a binding 
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Layering")
	float BlendInTime = 0.0f;
		
	// TODO: this should be a binding 
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Layering")
	float BlendOutTime = 0.0f;
	
	// TODO: this should be a binding 
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Layering")
	TObjectPtr<UCurveFloat> BlendCurve = nullptr;
	
	// TODO: this should be a binding 
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Layering")
	EAlphaBlendOption BlendOption = EAlphaBlendOption::Linear;

private:
	TArray<UE::UAF::Layering::FLayerStack_LayerEvent> EventsToProcess;
};

/** A trait that provides blend data based the outer layer stack */
USTRUCT(meta = (DisplayName = "Layer Data Provider", ShowTooltip=true, Hidden))
struct FUAFLayerDataProviderTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Layer")
	FUAFLayerProperties LayerProperties;

	// The index of this layer in its owning layer stack
	UPROPERTY(EditAnywhere, Category = "Layer", meta = (Inline))
	int32 LayerIndex = INDEX_NONE;

	// The layer name of this layer 
	UPROPERTY(EditAnywhere, Category = "Layer", meta = (Inline))
	FName LayerName = NAME_None;
	
	// The soft object path to the owning layer stack
	UPROPERTY(EditAnywhere, Category = "Layer", meta = (Inline))
	FString LayerStackPath;
	
	// If this is true we will allocate the input to allow for update/eval of cache only branches of the tree
	bool bCreateCacheInput = false;

	// This is only used to enqueue Cache Only layers to the graph update without actually blending them in in-place 
	UPROPERTY()
	FAnimNextTraitHandle CacheOnlyInput;

	// Latent pin support boilerplate
	#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
	GeneratorMacro(LayerProperties)

	GENERATE_TRAIT_LATENT_PROPERTIES(FUAFLayerDataProviderTraitSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
	#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR

};
