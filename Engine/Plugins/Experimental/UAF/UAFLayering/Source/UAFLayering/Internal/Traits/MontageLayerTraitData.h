// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraitCore/TraitSharedData.h"

#include "MontageLayerTraitData.generated.h"


/** A trait that supports modifying an existing layer based on montage runtime data */
USTRUCT(meta = (DisplayName = "Montage Layer Data", ShowTooltip=true, Hidden))
struct FUAFMontageLayerTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category = "Layer", meta = (Inline))
	bool bAutoEnableLayer = false;
	
	UPROPERTY(EditAnywhere, Category = "Layer", meta = (Inline))
	bool bAutoSetBlendTimes = false;
	
	// The montage slot name of this layer 
	UPROPERTY(EditAnywhere, Category = "Layer", meta = (Inline))
	FName SlotName = NAME_None;
	
	// The layer name of this layer 
	UPROPERTY(EditAnywhere, Category = "Layer", meta = (Inline))
	FName LayerName = NAME_None;
	
	// The soft object path to the owning layer stack
	UPROPERTY(EditAnywhere, Category = "Layer", meta = (Inline))
	FString LayerStackPath;
};
