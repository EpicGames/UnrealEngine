// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Graph/AnimNextAnimationGraph.h"
#include "UAFLayerStack.generated.h"

UCLASS(BlueprintType, MinimalAPI)
class UUAFLayerStack : public UUAFAnimGraph
{
	GENERATED_BODY()

public:
	friend class UUAFLayerStack_EditorData;
	friend class UUAFLayerStackFactory;
};
