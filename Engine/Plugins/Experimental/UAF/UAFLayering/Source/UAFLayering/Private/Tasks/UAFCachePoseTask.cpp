// Copyright Epic Games, Inc. All Rights Reserved.



#include "Tasks/UAFCachePoseTask.h"

#include "EvaluationVM/EvaluationVM.h"
#include "EvaluationVM/KeyframeState.h"

DEFINE_STAT(STAT_AnimNext_Task_CachePose);

FUAFCachePoseTask FUAFCachePoseTask::Make()
{
	return FUAFCachePoseTask();
}

void FUAFCachePoseTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;

	SCOPE_CYCLE_COUNTER(STAT_AnimNext_Task_CachePose);

	TUniquePtr<FKeyframeState> PoseToCache;
	const bool bCachedPose = VM.PopValue(KEYFRAME_STACK_NAME, PoseToCache);

	// TODO: Write that pose to a heap allocated property or stack memory
}
