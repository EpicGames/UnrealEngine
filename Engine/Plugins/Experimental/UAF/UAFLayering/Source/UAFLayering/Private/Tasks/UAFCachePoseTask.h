// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimNextStats.h"
#include "EvaluationVM/EvaluationTask.h"
#include "Stats/Stats.h"

#include "UAFCachePoseTask.generated.h"


DECLARE_CYCLE_STAT_EXTERN(TEXT("UAF Task: CachePose"), STAT_AnimNext_Task_CachePose, STATGROUP_AnimNext, UAFLAYERING_API);

#define UE_API UAFLAYERING_API

USTRUCT()
struct FUAFCachePoseTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FUAFCachePoseTask)
	
	static UE_API FUAFCachePoseTask Make();
	
	// Task Entry point
	UE_API virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;
};

#undef UE_API
