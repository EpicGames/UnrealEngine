// Copyright Epic Games, Inc. All Rights Reserved.


#include "CachePoseTrait.h"
#include "Tasks/UAFCachePoseTask.h"

namespace UE::UAF
{
AUTO_REGISTER_ANIM_TRAIT(FCachePoseTrait)

// Trait implementation boilerplate
#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
GeneratorMacro(IEvaluate) \

GENERATE_ANIM_TRAIT_IMPLEMENTATION(FCachePoseTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
#undef TRAIT_INTERFACE_ENUMERATOR

	
void FCachePoseTrait::PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const
{	
	Context.AppendTask(FUAFCachePoseTask::Make());
	
	IEvaluate::PostEvaluate(Context, Binding);
}
}
