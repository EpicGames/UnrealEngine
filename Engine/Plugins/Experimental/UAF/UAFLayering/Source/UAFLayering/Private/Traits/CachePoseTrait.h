// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Traits/CachePoseTraitData.h"
#include "TraitCore/Trait.h"
#include "TraitInterfaces/IEvaluate.h"

namespace UE::UAF
{
struct FCachePoseTrait : FAdditiveTrait, IEvaluate
{
	DECLARE_ANIM_TRAIT(FCachePoseTrait, FAdditiveTrait)

	struct FInstanceData : FTrait::FInstanceData
	{
	};

	using FSharedData = FUAFCachePoseTraitSharedData;

	// IEvaluate impl
	virtual void PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const override;
	
#if WITH_EDITOR
	virtual bool IsHidden() const override { return true; };
#endif
};
}
