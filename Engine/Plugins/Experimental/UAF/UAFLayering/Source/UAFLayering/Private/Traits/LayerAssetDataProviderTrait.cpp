// Copyright Epic Games, Inc. All Rights Reserved.

#include "LayerAssetDataProviderTrait.h"

#include "UAFLogging.h"
#include "Factory/AnimGraphFactory.h"
#include "TraitInterfaces/IBlendStack.h"
#include "TraitInterfaces/IGraphFactory.h"
#include "UAF/UAFAssetFactory.h"

namespace UE::UAF
{
	
	AUTO_REGISTER_ANIM_TRAIT(FLayerAssetDataProviderTrait)

	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IUpdate) \
		GeneratorMacro(IGarbageCollection) \
	
	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FLayerAssetDataProviderTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR
	#undef TRAIT_EVENT_ENUMERATOR


	void FLayerAssetDataProviderTrait::FInstanceData::Construct(const FExecutionContext& Context, const FTraitBinding& Binding)
	{
		FTrait::FInstanceData::Construct(Context, Binding);
			
		CachedAssetHandle = FGraphAssetHandle();
	}

	void FLayerAssetDataProviderTrait::PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		// Check if a new asset has been pushed 
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
			
		const FGraphAssetHandle& SharedDataAssetHandle = SharedData->GetGraphAssetHandle(Binding);
		if (SharedDataAssetHandle != InstanceData->CachedAssetHandle)
		{
			TTraitBinding<IBlendStack> BlendStackBinding;
			if(Binding.GetStackInterface<IBlendStack>(BlendStackBinding))
			{
				if (SharedDataAssetHandle.IsValid())
				{
					InstanceData->CachedAssetHandle = SharedDataAssetHandle;
					
					TArray<const UObject*> ReferencedObjects;
					SharedDataAssetHandle.GetPtr()->GetObjectReferences(ReferencedObjects);
					if(ReferencedObjects.Num() > 0)
					{
						FAnimNextFactoryParams FactoryParams = FAnimGraphFactory::GetDefaultParamsForAsset(SharedDataAssetHandle);
						const UUAFAnimGraph* AnimationGraph = IGraphFactory::GetOrBuildGraph(Context, Binding, SharedDataAssetHandle, FactoryParams);
						if(AnimationGraph)
						{
							IBlendStack::FGraphRequest NewGraphRequest;
							NewGraphRequest.BlendArgs = FAlphaBlendArgs(0.0f);
							NewGraphRequest.FactoryObject = ReferencedObjects[0];
							NewGraphRequest.AnimationGraph = AnimationGraph;
							NewGraphRequest.FactoryParams =  MoveTemp(FactoryParams);;

							BlendStackBinding.PushGraph(Context, MoveTemp(NewGraphRequest));
						}
						else
						{
							UAF_TRAIT_LOG(Error, TEXT("Failed to create graph!"));
						}
					}
				}
			}
			else
			{
				UAF_TRAIT_LOG(Error, TEXT("Blend Stack Trait is required but not found in stack!"));
			}
		}
			
		IUpdate::PreUpdate(Context, Binding, TraitState);
	}

	void FLayerAssetDataProviderTrait::AddReferencedObjects(const FExecutionContext& Context, const TTraitBinding<IGarbageCollection>& Binding, FReferenceCollector& Collector) const
	{
		IGarbageCollection::AddReferencedObjects(Context, Binding, Collector);
		
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		if(InstanceData->CachedAssetHandle.IsValid())
		{
			FGraphAssetHandle& AssetData = const_cast<FGraphAssetHandle&>(InstanceData->CachedAssetHandle);
			TWeakObjectPtr<const UScriptStruct> ScriptStruct = AssetData.GetScriptStruct();
			Collector.AddReferencedObjects(ScriptStruct, AssetData.GetMutableMemory());
		}
	}
	
}
