// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraitCore/Trait.h"
#include "TraitInterfaces/IGarbageCollection.h"
#include "TraitInterfaces/IUpdate.h"
#include "Traits/LayerAssetDataTraitData.h"

namespace UE::UAF
{
struct FLayerAssetDataProviderTrait : FAdditiveTrait, IUpdate, IGarbageCollection
{

	DECLARE_ANIM_TRAIT(FLayerAssetDataProviderTrait, FAdditiveTrait)

	struct FInstanceData : FTrait::FInstanceData
	{
		FGraphAssetHandle CachedAssetHandle;
		void Construct(const FExecutionContext& Context, const FTraitBinding& Binding);
	};
	
	using FSharedData = FUAFLayerAssetDataTraitSharedData;
	
	// IUpdate impl
	virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;
	
	// IGarbageCollection impl
	virtual void AddReferencedObjects(const FExecutionContext& Context, const TTraitBinding<IGarbageCollection>& Binding, FReferenceCollector& Collector) const override;
	
#if WITH_EDITOR
	virtual bool IsHidden() const override { return true; };
#endif
	
};
	
}
