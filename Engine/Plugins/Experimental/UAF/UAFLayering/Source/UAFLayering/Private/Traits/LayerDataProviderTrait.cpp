// Copyright Epic Games, Inc. All Rights Reserved.

#include "Traits/LayerDataProviderTrait.h"

#include "UAFLayerStack.h"
#include "UAFLogging.h"

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT(FLayerDataProviderTrait)

	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
	GeneratorMacro(IContinuousBlend) \
	GeneratorMacro(IUpdate) \
	GeneratorMacro(IHierarchy) \

	#define TRAIT_EVENT_ENUMERATOR(GeneratorMacro) \
	GeneratorMacro(FLayerDataProviderTrait::OnLayerEvent) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FLayerDataProviderTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR
	#undef TRAIT_EVENT_ENUMERATOR
	
	void FLayerDataProviderTrait::FInstanceData::Construct(const FExecutionContext& Context, const FTraitBinding& Binding)
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		LayerProperties = SharedData->GetLayerProperties(Binding);
		
		bLayerEnabled = LayerProperties.bLayerEnabled;
		TargetLayerWeight = LayerProperties.DesiredLayerWeight;
		EffectiveLayerWeight = LayerProperties.DesiredLayerWeight;
		LayerBlendInTime = LayerProperties.BlendInTime;
		LayerBlendOutTime = LayerProperties.BlendOutTime;
		
		if (LayerProperties.BlendCurve)
		{
			AlphaBlend.SetBlendOption(EAlphaBlendOption::Custom);
			AlphaBlend.SetCustomCurve(LayerProperties.BlendCurve);
		}
		else
		{
			AlphaBlend.SetBlendOption(LayerProperties.BlendOption);
		}

		if (bLayerEnabled)
		{
			AlphaBlend.SetBlendTime(LayerBlendInTime); 
			AlphaBlend.SetValueRange(0.0f, TargetLayerWeight);
		}
		else
		{
			AlphaBlend.SetBlendTime(LayerBlendOutTime); 
			AlphaBlend.SetValueRange(TargetLayerWeight, 0.0f);
		}
		
		AlphaBlend.SetAlpha(1.0f);
		
		if (SharedData->bCreateCacheInput)
		{
			FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
			if (!InstanceData->CacheOnlyInput.IsValid())
			{
				InstanceData->CacheOnlyInput = Context.AllocateNodeInstance(Binding, SharedData->CacheOnlyInput);
			}
		}
	}
	
	void FLayerDataProviderTrait::PostUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		
		// This currently mimics executing bindings, once bindings are implemented where this data lives is likely going to change
		// We only have to ensure that bindings will get executed after any child traits have finished updating, but before they are evaluating - so likely in PostUpdate or PreEval 
		FUAFLayerProperties& LayerData = InstanceData->LayerProperties;
		LayerData.ProcessEvents();
		
		// Once Bindings/Event have been processed and the desired data has been updated, we can now check if any of them have changed and handle it accordingly
		// (e.g. override existing data or blend them) 
		{
			// Update desired layer weight if it has changed 
			UpdateTargetLayerWeight(InstanceData, LayerData);
		
			// Update Blend In/Out times if they have changed 
			UpdateLayerBlendTimes(InstanceData, LayerData);
		
			// Check if the layer enabled flag has changed and handle it if so 
			HandleLayerActivation(InstanceData, LayerData);
		}
		
		// After any relevant blend data has been updated we can update the blend itself 
		InstanceData->AlphaBlend.Update(TraitState.GetDeltaTime());
		
		// Update the cached effective layer weight post blend update 
		InstanceData->EffectiveLayerWeight = InstanceData->AlphaBlend.GetBlendedValue();
		
		IUpdate::PostUpdate(Context, Binding, TraitState);
	}
	
	void FLayerDataProviderTrait::UpdateTargetLayerWeight(FInstanceData* InstanceData, const FUAFLayerProperties& LayerData) const
	{
		if (LayerData.DesiredLayerWeight != InstanceData->TargetLayerWeight)
		{
			InstanceData->TargetLayerWeight = LayerData.DesiredLayerWeight;
			
			// Update the layer weight blend to take into account the changed layer weight 
			if (InstanceData->AlphaBlend.IsComplete())
			{
				// If we are fully blended in with no active blend, set the weight directly 
				if (InstanceData->AlphaBlend.GetDesiredValue() > 0.0f)
				{
					InstanceData->AlphaBlend.SetDesiredValue(InstanceData->TargetLayerWeight);
					InstanceData->AlphaBlend.SetAlpha(1.0f);
				}
			}
			else
			{
				// If we are currently blending in, we have to adjust the value range 
				if (InstanceData->AlphaBlend.GetDesiredValue() > 0.0f)
				{
					InstanceData->AlphaBlend.SetValueRange(0.0f, InstanceData->TargetLayerWeight);
				}
				else
				{
					InstanceData->AlphaBlend.SetValueRange(InstanceData->TargetLayerWeight, 0.0f);
				}
			}
		}
	}
	
	void FLayerDataProviderTrait::UpdateLayerBlendTimes(FInstanceData* InstanceData, const FUAFLayerProperties& LayerData) const
	{
		// Update Layer Blend In Time 
		if (LayerData.BlendInTime != InstanceData->LayerBlendInTime)
		{
			InstanceData->LayerBlendInTime = LayerData.BlendInTime;
			
			// If we are currently blending in, we have to update the blend time
			if (!InstanceData->AlphaBlend.IsComplete() && InstanceData->AlphaBlend.GetDesiredValue() > 0.0f)
			{
				InstanceData->AlphaBlend.SetBlendTime(InstanceData->LayerBlendInTime);
			}
		}

		// Update Layer Blend Out Time
		if (LayerData.BlendOutTime != InstanceData->LayerBlendOutTime)
		{
			InstanceData->LayerBlendOutTime = LayerData.BlendOutTime;
			if (!InstanceData->AlphaBlend.IsComplete() && InstanceData->AlphaBlend.GetDesiredValue() <= 0.0f)
			{
				InstanceData->AlphaBlend.SetBlendTime(InstanceData->LayerBlendOutTime);
			}
		}
	}
	
	void FLayerDataProviderTrait::HandleLayerActivation(FInstanceData* InstanceData, const FUAFLayerProperties& LayerData) const
	{
		if (LayerData.bLayerEnabled != InstanceData->bLayerEnabled)
		{
			InstanceData->bLayerEnabled = LayerData.bLayerEnabled;
			
			// If the layer is now enabled, we are blending in. 
			const bool bBlendingIn = InstanceData->bLayerEnabled;
			InstanceData->AlphaBlend.SetBlendTime(bBlendingIn ? InstanceData->LayerBlendInTime : InstanceData->LayerBlendOutTime); 
			InstanceData->AlphaBlend.SetValueRange(InstanceData->EffectiveLayerWeight, bBlendingIn ? LayerData.DesiredLayerWeight : 0.0f);
		}
	}

	float FLayerDataProviderTrait::GetBlendWeight(const FExecutionContext& Context, const TTraitBinding<IContinuousBlend>& Binding, int32 ChildIndex) const
	{		
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		if (ChildIndex == 0)
		{
			return 1.0f - InstanceData->EffectiveLayerWeight;
		}
		
		if (ChildIndex == 1)
		{
			return InstanceData->EffectiveLayerWeight;
		}
		
		UAF_TRAIT_LOG(Warning, TEXT("Requested blend weight for invalid child index %d."), ChildIndex);
		return -1.0f; // Invalid Child Index 
	}

	ETraitStackPropagation FLayerDataProviderTrait::OnLayerEvent(const FExecutionContext& Context, FTraitBinding& Binding, Layering::FLayerStack_LayerEvent& Event) const
	{
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		
		const FName LayerName = SharedData->LayerName;
		const int32 LayerIndex = SharedData->LayerIndex;
		const FSoftObjectPath& LayerStackPath = SharedData->LayerStackPath;
		if ( (LayerName == Event.LayerName || LayerIndex == Event.LayerIndex) && LayerStackPath == Event.LayerStackAssetPath)
		{
			InstanceData->LayerProperties.AddEvent(Event);
			Event.bAutoConsumeEvent ? Event.MarkConsumed() : Event.MarkHandled();
		}
		
		return ETraitStackPropagation::Continue;
	}

	uint32 FLayerDataProviderTrait::GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		return InstanceData->CacheOnlyInput.IsValid() ? 1 : 0;
	}

	void FLayerDataProviderTrait::GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const
	{
		// Only do this if cache only mode 
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		if (InstanceData->CacheOnlyInput.IsValid())
		{
			Children.Add(InstanceData->CacheOnlyInput);
		}
	}
}
