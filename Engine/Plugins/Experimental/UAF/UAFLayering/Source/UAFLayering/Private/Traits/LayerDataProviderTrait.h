// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Traits/LayerDataProviderTraitData.h"
#include "TraitCore/Trait.h"
#include "TraitInterfaces/IContinuousBlend.h"
#include "TraitInterfaces/IHierarchy.h"
#include "TraitInterfaces/IUpdate.h"

namespace UE::UAF
{
struct FLayerDataProviderTrait : FAdditiveTrait, IUpdate, IContinuousBlend, IHierarchy
{
	DECLARE_ANIM_TRAIT(FLayerDataProviderTrait, FAdditiveTrait)

	struct FInstanceData : FTrait::FInstanceData
	{
		// If the layer is currently enabled or not. 
		bool bLayerEnabled = true;
		
		// The cached effective layer weight - the actual weight used when applying this layer based on the target weight and any active blends
		float EffectiveLayerWeight = 1.0f;
		
		// The target weight of this layer. This is the desired weight to apply for this layer, however if the layer is currently blending in/out 
		// this might not be the weight that gets applied - see EffectiveLayerWeight. 
		float TargetLayerWeight = 1.0f;
		
		// The blend in time used when blending this layer in
		float LayerBlendInTime = 0.0f;
		
		// The blend out time used when blending this layer out 
		float LayerBlendOutTime = 0.0f;
		
		// Trait handle to execute cache only parts of the graph
		FTraitPtr CacheOnlyInput;
		
		// This updated the discrete blend when a layer blends in or out 
		FAlphaBlend AlphaBlend;
		
		// This layer data is likely not going to live here once bindings are implemented
		// This contains all the bound / desired layer properties versus the rest of the instance data containing the currently used data based on
		// any blending between properties and to check if data has changed between frames 
		FUAFLayerProperties LayerProperties;
		
		void Construct(const FExecutionContext& Context, const FTraitBinding& Binding);
	};

	using FSharedData = FUAFLayerDataProviderTraitSharedData;
	
	// IUpdate imp
	virtual void PostUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;
	
	// IContinuousBlend impl - this allows us to override the weight for any lower blend trait 
	virtual float GetBlendWeight(const FExecutionContext& Context, const TTraitBinding<IContinuousBlend>& Binding, int32 ChildIndex) const override;

	// Event Handling
	ETraitStackPropagation OnLayerEvent(const FExecutionContext& Context, FTraitBinding& Binding, Layering::FLayerStack_LayerEvent& Event) const;
	
	// IHierarchy Impl
	virtual uint32 GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const override;
	virtual void GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const override;
	
#if WITH_EDITOR
	virtual bool IsHidden() const override { return true; };
#endif
	
private:
	void UpdateTargetLayerWeight(FInstanceData* InstanceData, const FUAFLayerProperties& LayerData) const;
	void UpdateLayerBlendTimes(FInstanceData* InstanceData, const FUAFLayerProperties& LayerData) const;
	void HandleLayerActivation(FInstanceData* InstanceData, const FUAFLayerProperties& LayerData) const;
	
};
}
