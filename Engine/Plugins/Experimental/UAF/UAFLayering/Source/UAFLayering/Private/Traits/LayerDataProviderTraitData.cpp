// Copyright Epic Games, Inc. All Rights Reserved.

#include "Traits/LayerDataProviderTraitData.h"

void FUAFLayerProperties::AddEvent(const UE::UAF::Layering::FLayerStack_LayerEvent& Event)
{
	EventsToProcess.Add(Event);
}

void FUAFLayerProperties::ProcessEvents()
{
	using namespace UE::UAF::Layering;
	
	for (const FLayerStack_LayerEvent& Event : EventsToProcess)
	{
		switch (Event.Action)
		{
		case ELayerEventAction::EnableLayer:
			bLayerEnabled = true;
			break;
		case ELayerEventAction::DisableLayer:
			bLayerEnabled = false;
			break;
		case ELayerEventAction::SetFloatValue:
			if (Event.PropertyToSet.IsEqual(FLayerStack_LayerEvent::LayerWeightProperty))
			{
				DesiredLayerWeight = Event.FloatValue;
			}
			else if (Event.PropertyToSet.IsEqual(FLayerStack_LayerEvent::BlendInTimeProperty))
			{
				BlendInTime = Event.FloatValue;
			}
			else if (Event.PropertyToSet.IsEqual(FLayerStack_LayerEvent::BlendOutTimeProperty))
			{
				BlendOutTime = Event.FloatValue;
			}
			break;
		case ELayerEventAction::None:
		default:
			break;
		}
	}
	
	EventsToProcess.Empty();
}
