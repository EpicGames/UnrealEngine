// Copyright Epic Games, Inc. All Rights Reserved.

#include "Traits/MontageLayerDataTrait.h"

#include "UAFLayeringTypes.h"
#include "Injection/UAFMontageComponent.h"
#include "Module/AnimNextModuleInstance.h"

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT(FMontageLayerDataTrait)

	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IUpdate) \
	
	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FMontageLayerDataTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR
	#undef TRAIT_EVENT_ENUMERATOR
	

	void FMontageLayerDataTrait::PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		
		const bool bAutoEnableLayer = SharedData->bAutoEnableLayer;
		const bool bAutoSetBlendTimes = SharedData->bAutoSetBlendTimes;
		if (bAutoEnableLayer || bAutoSetBlendTimes)
		{
			if (FAnimNextModuleInstance* ModuleInstance = Context.GetRootGraphInstance().GetModuleInstance())
			{
				FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
				FUAFMontageComponent& MontageComponent = ModuleInstance->GetOrAddComponent<FUAFMontageComponent>();
				const bool bSlotActive = MontageComponent.IsSlotActive(SharedData->SlotName);
				
				// Enable or disable the layer based on if a montage is currently playing in the relevant slot 
				if (bAutoEnableLayer && (bSlotActive != InstanceData->bLastEnableState || TraitState.IsNewlyRelevant()))
				{
					InstanceData->bLastEnableState = bSlotActive;
				
					TSharedPtr<Layering::FLayerStack_LayerEvent> Event = MakeTraitEvent<Layering::FLayerStack_LayerEvent>(SharedData->LayerName, SharedData->LayerStackPath);
					Event->Action = bSlotActive ? Layering::ELayerEventAction::EnableLayer : Layering::ELayerEventAction::DisableLayer;
					Context.RaiseOutputTraitEvent(Event);
				}

				if (bAutoSetBlendTimes && bSlotActive)
				{
					const TObjectPtr<const UAnimMontage> PlayingMontage = MontageComponent.GetActiveMontageForSlot(SharedData->SlotName);
					if (PlayingMontage && InstanceData->LastActiveMontage != PlayingMontage)
					{
						InstanceData->LastActiveMontage = PlayingMontage;
						
						TSharedPtr<Layering::FLayerStack_LayerEvent> SetBlendInTimeEvent = MakeTraitEvent<Layering::FLayerStack_LayerEvent>(SharedData->LayerName, SharedData->LayerStackPath);
						SetBlendInTimeEvent->Action = Layering::ELayerEventAction::SetFloatValue;
						SetBlendInTimeEvent->PropertyToSet = Layering::FLayerStack_LayerEvent::BlendInTimeProperty;
						SetBlendInTimeEvent->FloatValue = PlayingMontage->BlendIn.GetBlendTime();
						Context.RaiseOutputTraitEvent(SetBlendInTimeEvent);
						
						TSharedPtr<Layering::FLayerStack_LayerEvent> SetBlendOutTimeEvent = MakeTraitEvent<Layering::FLayerStack_LayerEvent>(SharedData->LayerName, SharedData->LayerStackPath);
						SetBlendOutTimeEvent->Action = Layering::ELayerEventAction::SetFloatValue;
						SetBlendOutTimeEvent->PropertyToSet = Layering::FLayerStack_LayerEvent::BlendOutTimeProperty;
						SetBlendOutTimeEvent->FloatValue = PlayingMontage->BlendOut.GetBlendTime();
						Context.RaiseOutputTraitEvent(SetBlendOutTimeEvent);
					}
				}
				
				if (!bSlotActive && InstanceData->LastActiveMontage.IsValid())
				{
					InstanceData->LastActiveMontage = nullptr;
				}
			}
		}
		
		IUpdate::PreUpdate(Context, Binding, TraitState);
	}
}
