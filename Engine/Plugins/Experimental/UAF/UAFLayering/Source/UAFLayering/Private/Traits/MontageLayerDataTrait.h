// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraitCore/Trait.h"
#include "TraitInterfaces/IUpdate.h"
#include "Traits/MontageLayerTraitData.h"

namespace UE::UAF
{
	struct FMontageLayerDataTrait : FAdditiveTrait, IUpdate
	{
		DECLARE_ANIM_TRAIT(FMontageLayerDataTrait, FAdditiveTrait)

		struct FInstanceData : FTrait::FInstanceData
		{
			bool bLastEnableState = false;
			TWeakObjectPtr<const UAnimMontage> LastActiveMontage = nullptr;
		};
	
		using FSharedData = FUAFMontageLayerTraitSharedData;
	
		// IUpdate impl
		virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;

#if WITH_EDITOR
		virtual bool IsHidden() const override { return true; };
#endif
	
	};
	
}
