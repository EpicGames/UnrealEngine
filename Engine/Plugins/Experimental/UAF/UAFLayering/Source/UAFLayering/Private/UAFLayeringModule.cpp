// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAFLayeringModule.h"

#include "Factory/AnimNextFactoryParams.h"
#include "RigVMCore/RigVMRegistry.h"
#include "Traits/LayerDataProviderTraitData.h"

#define LOCTEXT_NAMESPACE "FUAFLayeringModule"

void FUAFLayeringModule::StartupModule()
{
	static UScriptStruct* const AllowedStructTypes[] =
	{
		FUAFLayerProperties::StaticStruct(),
		FAnimNextFactoryParams::StaticStruct(),
	};
	
	FRigVMRegistry::Get().RegisterStructTypes(AllowedStructTypes);
}

void FUAFLayeringModule::ShutdownModule()
{
	
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FUAFLayeringModule, UAFLayering)
