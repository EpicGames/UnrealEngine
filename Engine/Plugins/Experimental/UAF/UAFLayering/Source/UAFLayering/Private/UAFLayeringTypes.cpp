// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAFLayeringTypes.h"

namespace UE::UAF::Layering
{
	const FName FLayerStack_LayerEvent::LayerWeightProperty(TEXT("LayerWeight"));
	const FName FLayerStack_LayerEvent::BlendInTimeProperty(TEXT("BlendInTime"));
	const FName FLayerStack_LayerEvent::BlendOutTimeProperty(TEXT("BlendOutTime"));
	
}
