// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAFLayeringUtils.h"

#include "UAFLayeringTypes.h"
#include "Component/AnimNextComponent.h"
#include "TraitCore/TraitEvent.h"

void UUAFLayeringUtils::DisableLayer(UUAFComponent* UAFComponent, FName LayerName, const TSoftObjectPtr<UUAFLayerStack> LayerStackPath)
{
	if (!UAFComponent || LayerName == NAME_None || LayerStackPath.IsNull())
	{
		return;
	}
	
	TSharedPtr<UE::UAF::Layering::FLayerStack_LayerEvent> Event = MakeTraitEvent<UE::UAF::Layering::FLayerStack_LayerEvent>(LayerName, LayerStackPath.ToSoftObjectPath());
	Event->Action = UE::UAF::Layering::ELayerEventAction::DisableLayer;
	UAFComponent->QueueInputTraitEvent(Event);
}

void UUAFLayeringUtils::EnableLayer(UUAFComponent* UAFComponent, FName LayerName, const TSoftObjectPtr<UUAFLayerStack> LayerStackPath)
{
	if (!UAFComponent || LayerName == NAME_None || LayerStackPath.IsNull())
	{
		return;
	}
	
	TSharedPtr<UE::UAF::Layering::FLayerStack_LayerEvent> Event = MakeTraitEvent<UE::UAF::Layering::FLayerStack_LayerEvent>(LayerName, LayerStackPath.ToSoftObjectPath());
	Event->Action = UE::UAF::Layering::ELayerEventAction::EnableLayer;
	UAFComponent->QueueInputTraitEvent(Event);
}

void UUAFLayeringUtils::SetLayerWeight(UUAFComponent* UAFComponent, FName LayerName, const TSoftObjectPtr<UUAFLayerStack> LayerStackPath, const float Weight)
{
	if (!UAFComponent || LayerName == NAME_None || LayerStackPath.IsNull())
	{
		return;
	}
	
	TSharedPtr<UE::UAF::Layering::FLayerStack_LayerEvent> Event = MakeTraitEvent<UE::UAF::Layering::FLayerStack_LayerEvent>(LayerName, LayerStackPath.ToSoftObjectPath());
	Event->Action = UE::UAF::Layering::ELayerEventAction::SetFloatValue;
	Event->PropertyToSet = UE::UAF::Layering::FLayerStack_LayerEvent::LayerWeightProperty;
	Event->FloatValue = Weight;
	UAFComponent->QueueInputTraitEvent(Event);
}

void UUAFLayeringUtils::DisableLayerByIndex(UUAFComponent* UAFComponent, int32 LayerIndex, const TSoftObjectPtr<UUAFLayerStack> LayerStackPath)
{
	if (!UAFComponent || LayerIndex == INDEX_NONE || LayerStackPath.IsNull())
	{
		return;
	}
	
	TSharedPtr<UE::UAF::Layering::FLayerStack_LayerEvent> Event = MakeTraitEvent<UE::UAF::Layering::FLayerStack_LayerEvent>(LayerIndex, LayerStackPath.ToSoftObjectPath());
	Event->Action = UE::UAF::Layering::ELayerEventAction::DisableLayer;
	UAFComponent->QueueInputTraitEvent(Event);
}

void UUAFLayeringUtils::EnableLayerByIndex(UUAFComponent* UAFComponent, int32 LayerIndex, const TSoftObjectPtr<UUAFLayerStack> LayerStackPath)
{
	if (!UAFComponent || LayerIndex == INDEX_NONE || LayerStackPath.IsNull())
	{
		return;
	}
	
	TSharedPtr<UE::UAF::Layering::FLayerStack_LayerEvent> Event = MakeTraitEvent<UE::UAF::Layering::FLayerStack_LayerEvent>(LayerIndex, LayerStackPath.ToSoftObjectPath());
	Event->Action = UE::UAF::Layering::ELayerEventAction::EnableLayer;
	UAFComponent->QueueInputTraitEvent(Event);
}

void UUAFLayeringUtils::SetLayerWeightByIndex(UUAFComponent* UAFComponent, int32 LayerIndex, const TSoftObjectPtr<UUAFLayerStack> LayerStackPath, const float Weight)
{
	if (!UAFComponent || LayerIndex == INDEX_NONE || LayerStackPath.IsNull())
	{
		return;
	}
	
	TSharedPtr<UE::UAF::Layering::FLayerStack_LayerEvent> Event = MakeTraitEvent<UE::UAF::Layering::FLayerStack_LayerEvent>(LayerIndex, LayerStackPath.ToSoftObjectPath());
	Event->Action = UE::UAF::Layering::ELayerEventAction::SetFloatValue;
	Event->PropertyToSet =  UE::UAF::Layering::FLayerStack_LayerEvent::LayerWeightProperty;
	Event->FloatValue = Weight;
	UAFComponent->QueueInputTraitEvent(Event);
}
