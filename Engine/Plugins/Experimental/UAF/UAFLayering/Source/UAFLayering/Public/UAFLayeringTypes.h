// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraitCore/TraitEvent.h"

namespace UE::UAF::Layering
{
	
enum class ELayerEventAction : uint8
{
	None,
	EnableLayer,
	DisableLayer,
	SetFloatValue
};
	
struct FLayerStack_LayerEvent : public FAnimNextTraitEvent
{
	DECLARE_ANIM_TRAIT_EVENT(FLayerStack_LayerEvent, FAnimNextTraitEvent)

	FLayerStack_LayerEvent(FName InLayerName, const FSoftObjectPath& InObjectPath)
		: LayerName(InLayerName)
		, LayerStackAssetPath(InObjectPath)
	{}
	
	FLayerStack_LayerEvent(int32 InLayerIndex, const FSoftObjectPath& InObjectPath)
		: LayerIndex(InLayerIndex)
		, LayerStackAssetPath(InObjectPath)
	{}
	
	// The name of the layer to affect 
	const FName LayerName = NAME_None;
	
	// The index in the layer stack for the layer to affect
	const int32 LayerIndex = INDEX_NONE;

	// The soft object path of the layer stack asset the layer exists in 
	const FSoftObjectPath LayerStackAssetPath = FSoftObjectPath();

	// The action to perform 
	ELayerEventAction Action = ELayerEventAction::None;

	// The name of the property to change if applicable 
	FName PropertyToSet = NAME_None;

	// The float value to set, if applicable
	float FloatValue = 0.0f;
	
	// If this event should be consumed by the first found layer that matches the description
	// Per default the event will be passed through the UAF system and every layer gets the option to react to it 
	bool bAutoConsumeEvent = false;
	
	static const FName LayerWeightProperty;	
	static const FName BlendInTimeProperty;	
	static const FName BlendOutTimeProperty;
};
	
}
