// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "UAFLayeringUtils.generated.h"

class UUAFLayerStack;
class UUAFComponent; 

#define UE_API UAFLAYERING_API

UCLASS(MinimalAPI)
class UUAFLayeringUtils : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
	
	// Enables the specified layer by name in the specified Layer Stack
	UFUNCTION(BlueprintCallable, Category = "UAF|Layering", meta = (ScriptMethod))
	static UE_API void EnableLayer(UUAFComponent* UAFComponent, FName LayerName, const TSoftObjectPtr<UUAFLayerStack> LayerStackPath);
	
	// Disables the specified layer by name in the specified Layer Stack
	UFUNCTION(BlueprintCallable, Category = "UAF|Layering", meta = (ScriptMethod))
	static UE_API void DisableLayer(UUAFComponent* UAFComponent, FName LayerName, const TSoftObjectPtr<UUAFLayerStack> LayerStackPath);
	
	// Sets the weight of the specified layer by name in the specified Layer Stack
	UFUNCTION(BlueprintCallable, Category = "UAF|Layering", meta = (ScriptMethod))
	static UE_API void SetLayerWeight(UUAFComponent* UAFComponent, FName LayerName, const TSoftObjectPtr<UUAFLayerStack> LayerStackPath, const float Weight);
	
	// Enables the specified layer by index in the specified Layer Stack
	UFUNCTION(BlueprintCallable, Category = "UAF|Layering", meta = (ScriptMethod))
	static UE_API void EnableLayerByIndex(UUAFComponent* UAFComponent, int32 LayerIndex, const TSoftObjectPtr<UUAFLayerStack> LayerStackPath);
	
	// Disables the specified layer by index in the specified Layer Stack
	UFUNCTION(BlueprintCallable, Category = "UAF|Layering", meta = (ScriptMethod))
	static UE_API void DisableLayerByIndex(UUAFComponent* UAFComponent, int32 LayerIndex, const TSoftObjectPtr<UUAFLayerStack> LayerStackPath);
	
	// Sets the weight of the specified layer by index in the specified Layer Stack
	UFUNCTION(BlueprintCallable, Category = "UAF|Layering", meta = (ScriptMethod))
	static UE_API void SetLayerWeightByIndex(UUAFComponent* UAFComponent, int32 LayerIndex, const TSoftObjectPtr<UUAFLayerStack> LayerStackPath, const float Weight);
};

#undef UE_API
