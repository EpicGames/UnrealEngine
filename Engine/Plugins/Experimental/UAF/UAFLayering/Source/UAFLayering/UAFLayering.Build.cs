// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class UAFLayering : ModuleRules
{
	public UAFLayering(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"UAF"
				// ... add other public dependencies that you statically link with here ...
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
				"UAFAnimGraph",
				"RigVM",
				"UAFAnimGraph"
				// ... add private dependencies that you statically link with here ...	
			}
			);
		
		
		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
			);
	}
}
