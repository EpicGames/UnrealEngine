// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Factories/Factory.h"
#include "UAFLayerStackFactory.generated.h"

UCLASS(MinimalAPI)
class UUAFLayerStackFactory : public UFactory
{
	GENERATED_BODY()

	UUAFLayerStackFactory();

	virtual bool ConfigureProperties() override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext) override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
};