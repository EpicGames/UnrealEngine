// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAFLayerStackAssetDefinition.h"
#include "UAFLayerStack.h"
#include "IWorkspaceEditorModule.h"
#include "Modules/ModuleManager.h"
#include "Workspace/AnimNextWorkspaceFactory.h"

#define LOCTEXT_NAMESPACE "AnimNextAssetDefinitions"

FText UAssetDefinition_UAFLayerStack::GetAssetDisplayName() const
{
	return LOCTEXT("UAFLayerStack", "UAF Layer Stack"); 
}

FLinearColor UAssetDefinition_UAFLayerStack::GetAssetColor() const
{ 
	return FLinearColor(FColor(148, 49, 145));
}

TSoftClassPtr<UObject> UAssetDefinition_UAFLayerStack::GetAssetClass() const
{ 
	return UUAFLayerStack::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_UAFLayerStack::GetAssetCategories() const
{
	static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Animation, LOCTEXT("UAFSubMenu", "Animation Framework")) };
	return Categories;
}

EAssetCommandResult UAssetDefinition_UAFLayerStack::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	using namespace UE::UAF::Editor;
	using namespace UE::Workspace;

	for (UUAFLayerStack* Asset : OpenArgs.LoadObjects<UUAFLayerStack>())
	{
		IWorkspaceEditorModule& WorkspaceEditorModule = FModuleManager::Get().LoadModuleChecked<IWorkspaceEditorModule>("WorkspaceEditor");
		WorkspaceEditorModule.OpenWorkspaceForObject(Asset, EOpenWorkspaceMethod::Default, UAnimNextWorkspaceFactory::StaticClass());
	}

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE