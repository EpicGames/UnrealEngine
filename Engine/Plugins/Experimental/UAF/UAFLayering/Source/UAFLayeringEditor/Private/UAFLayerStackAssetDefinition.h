// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"
#include "UAFLayerStackAssetDefinition.generated.h"

UCLASS()
class UAssetDefinition_UAFLayerStack : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// Begin UAssetDefinition Interface
	virtual FText GetAssetDisplayName() const override; 
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// End UAssetDefinition Interface
};

