// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAFLayerStackFactory.h"
#include "UAFLayerStack.h"
#include "UAFLayerStack_EditorData.h"
#include "UAFCompilationScope.h"

UUAFLayerStackFactory::UUAFLayerStackFactory()
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UUAFLayerStack::StaticClass();
}

bool UUAFLayerStackFactory::ConfigureProperties()
{
	return true;
}

UObject* UUAFLayerStackFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext)
{
	EObjectFlags FlagsToUse = Flags | RF_Public | RF_Standalone | RF_Transactional | RF_LoadCompleted;
	if (InParent == GetTransientPackage())
	{
		FlagsToUse &= ~RF_Standalone;
	}

	// Create LayerStack Asset
	UUAFLayerStack* LayerStack = NewObject<UUAFLayerStack>(InParent, Class, Name, FlagsToUse);

	// Create related Editor Data
	const FName EditorDataName = FName(Name.ToString() + "_Layering_EditorData");
	UUAFLayerStack_EditorData* EditorData = NewObject<UUAFLayerStack_EditorData>(LayerStack, EditorDataName, RF_Transactional);

	// Set the layer controllers editor data reference
	LayerStack->EditorData = EditorData;

	EditorData->Initialize(/*bRecompileVM*/false);

	UE::UAF::UncookedOnly::Compilation::RequestAssetCompilation(LayerStack);
	ensure(!EditorData->bErrorsDuringCompilation);

	return LayerStack;
}

UObject* UUAFLayerStackFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return FactoryCreateNew(Class, InParent, Name, Flags, Context, Warn, NAME_None);
}
