// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAFLayeringEditorMode.h"

#define LOCTEXT_NAMESPACE "UAFLayeringEditorMode"

const FEditorModeID UUAFLayeringEditorMode::EM_UAFLayeringMode("UAFLayeringEditorMode");

UUAFLayeringEditorMode::UUAFLayeringEditorMode()
{
	Info = FEditorModeInfo(UUAFLayeringEditorMode::EM_UAFLayeringMode,
	LOCTEXT("UAFLayeringEditorModeName", "UAFLayeringEditorMode"),
	FSlateIcon(),
	false);
}

#undef LOCTEXT_NAMESPACE // "UAFLayeringEditorMode"
