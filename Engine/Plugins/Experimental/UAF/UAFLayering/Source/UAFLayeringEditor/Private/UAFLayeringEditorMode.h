// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Tools/UEdMode.h"
#include "UAFLayeringEditorMode.generated.h"

UCLASS()
class UUAFLayeringEditorMode : public UEdMode
{
	GENERATED_BODY()
public:
	const static FEditorModeID EM_UAFLayeringMode;
	
	UUAFLayeringEditorMode();
};
