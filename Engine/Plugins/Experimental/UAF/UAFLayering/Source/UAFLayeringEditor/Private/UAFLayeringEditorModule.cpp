// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAFLayeringEditorModule.h"

#include "ContextObjectStore.h"
#include "EditorModeManager.h"
#include "EditorUtils.h"
#include "IAnimNextEditorModule.h"
#include "IWorkspaceEditor.h"
#include "UAFLayeringEditorMode.h"
#include "UAFLayeringStyle.h"
#include "UAFLayerStack.h"
#include "Layers/UAFLayer.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "Widgets/SLayerStack.h"
#include "Workspace/LayerStackItemDetails.h"
#include "Workspace/LayerStackViewportController.h"
#include "Workspace/UAFLayerStackWorkspaceOutlinerData.h"


#define LOCTEXT_NAMESPACE "FUAFLayeringEditorModule"

namespace UE::UAF::LayeringEditor
{
void FUAFLayeringEditorModule::StartupModule()
{
	FUAFLayeringStyle::Get();

	UE::UAF::Editor::IAnimNextEditorModule& UAFEditorModule = FModuleManager::Get().LoadModuleChecked<UE::UAF::Editor::IAnimNextEditorModule>("UAFEditor");
	UAFEditorModule.AddWorkspaceSupportedAssetClass(UUAFLayerStack::StaticClass()->GetClassPathName());

	// Build Document Args
	UE::Workspace::FObjectDocumentArgs LayerStackDocumentArgs;
	LayerStackDocumentArgs.OnMakeDocumentWidget.BindRaw(this, &FUAFLayeringEditorModule::MakeDocumentWidget);
	LayerStackDocumentArgs.OnGetDocumentBreadcrumbTrail.BindRaw(this, &FUAFLayeringEditorModule::GetBreadcrumbTrail);
	LayerStackDocumentArgs.SpawnLocation = UE::Workspace::WorkspaceTabs::TopMiddleDocumentArea;
	LayerStackDocumentArgs.DocumentEditorMode = UUAFLayeringEditorMode::EM_UAFLayeringMode;
	LayerStackDocumentArgs.OnGetTabIcon = UE::Workspace::FOnGetTabIcon::CreateLambda([](const UE::Workspace::FWorkspaceEditorContext& InContext)
		{
			return FUAFLayeringStyle::Get().GetBrush(TEXT("ClassIcon.UAFLayerStack"));
		});

	UE::Workspace::IWorkspaceEditorModule& WorkspaceEditorModule = FModuleManager::Get().LoadModuleChecked<UE::Workspace::IWorkspaceEditorModule>("WorkspaceEditor");
	WorkspaceEditorModule.RegisterObjectDocumentType(UUAFLayerStack::StaticClass()->GetClassPathName(), LayerStackDocumentArgs);
	WorkspaceEditorModule.RegisterWorkspaceItemDetails(UE::Workspace::FOutlinerItemDetailsId(FUAFLayerStackWorkspaceOutlinerData::StaticStruct()->GetFName()), MakeShared<UE::UAF::LayeringEditor::ULayerStackItemDetails>());
	WorkspaceEditorModule.RegisterWorkspaceItemDetails(UE::Workspace::FOutlinerItemDetailsId(FUAFLayerStackLayerOutlinerData::StaticStruct()->GetFName()), MakeShared<UE::UAF::LayeringEditor::ULayerStackLayerItemDetails>());
	
	WorkspaceEditorModule.RegisterViewportControllerFactory(UUAFLayerStack::StaticClass(), []()
		{
			return MakeUnique<FLayerStackViewportController>();
		});
}

void FUAFLayeringEditorModule::ShutdownModule()
{
	if (UObjectInitialized())
	{
		// Unregister document type for layer stack class from workspace 
		if (UE::Workspace::IWorkspaceEditorModule* WorkspaceEditorModule = FModuleManager::Get().GetModulePtr<UE::Workspace::IWorkspaceEditorModule>("WorkspaceEditor"))
		{
			WorkspaceEditorModule->UnregisterObjectDocumentType(UUAFLayerStack::StaticClass()->GetClassPathName());
			WorkspaceEditorModule->UnregisterWorkspaceItemDetails(UE::Workspace::FOutlinerItemDetailsId(FUAFLayerStackWorkspaceOutlinerData::StaticStruct()->GetFName()));
			WorkspaceEditorModule->UnregisterWorkspaceItemDetails(UE::Workspace::FOutlinerItemDetailsId(FUAFLayerStackLayerOutlinerData::StaticStruct()->GetFName()));
			WorkspaceEditorModule->UnregisterViewportControllerFactory(UUAFLayerStack::StaticClass());
		}

		// Unregister layer stack class from UAFEditor
		if (UE::UAF::Editor::IAnimNextEditorModule* UAFEditorModule = FModuleManager::Get().GetModulePtr<UE::UAF::Editor::IAnimNextEditorModule>("UAFEditor"))
		{
			UAFEditorModule->RemoveWorkspaceSupportedAssetClass(UUAFLayerStack::StaticClass()->GetClassPathName());
		}
	}
}

TSharedRef<SWidget> FUAFLayeringEditorModule::MakeDocumentWidget(const UE::Workspace::FWorkspaceEditorContext& InContext)
{
	if (UUAFLayerStack* LayerStack = InContext.Document.GetTypedObject<UUAFLayerStack>())
	{
		return SNew(UE::UAF::Layering::SLayerStack)
			.LayerStack(LayerStack)
			.WorkspaceEditor(InContext.WorkspaceEditor);
	}

	return SNullWidget::NullWidget;
}

void FUAFLayeringEditorModule::GetBreadcrumbTrail(const UE::Workspace::FWorkspaceEditorContext& InContext, TArray<TSharedPtr<UE::Workspace::FWorkspaceBreadcrumb>>& OutBreadcrumbs)
{
	UUAFLayerStack* LayerStack = InContext.Document.GetTypedObject<UUAFLayerStack>();
	if (!LayerStack)
	{
		UE_LOGF(LogAnimation, Error, "Tried to get breadcrumb trail for LayerStack asset but object was not a LayerStack");
		return;
	}

	// Create Breadcrumb
	const TSharedPtr<UE::Workspace::FWorkspaceBreadcrumb>& BreadCrumb = OutBreadcrumbs.Add_GetRef(MakeShared<UE::Workspace::FWorkspaceBreadcrumb>());

	// Set Breadcrumb Label
	BreadCrumb->OnGetLabel = UE::Workspace::FWorkspaceBreadcrumb::FOnGetBreadcrumbLabel::CreateLambda([LayerStackName = LayerStack->GetFName()]
		{
			return FText::FromName(LayerStackName);
		});

	// Set Breadcrumb dirty flag 
	TWeakObjectPtr<UUAFLayerStack> WeakLayerStack = LayerStack;
	BreadCrumb->CanSave = UE::Workspace::FWorkspaceBreadcrumb::FCanSaveBreadcrumb::CreateLambda([WeakLayerStack]
		{
			return WeakLayerStack.IsValid() && WeakLayerStack->GetPackage()->IsDirty();
		});

	// Set breadcrumb click behavior
	TWeakPtr<UE::Workspace::IWorkspaceEditor> WeakWorkspaceEditor = InContext.WorkspaceEditor;
	BreadCrumb->OnClicked = UE::Workspace::FWorkspaceBreadcrumb::FOnBreadcrumbClicked::CreateLambda([WeakWorkspaceEditor, Export = InContext.Document.Export]
		{
			if (const TSharedPtr<UE::Workspace::IWorkspaceEditor> SharedWorkspaceEditor = WeakWorkspaceEditor.Pin())
			{
				SharedWorkspaceEditor->OpenExports({ Export });
			}
		});
}

IMPLEMENT_MODULE(FUAFLayeringEditorModule, UAFLayeringEditor)

}

#undef LOCTEXT_NAMESPACE
	
