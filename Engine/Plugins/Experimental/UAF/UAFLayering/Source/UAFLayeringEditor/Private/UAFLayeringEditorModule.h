// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "IWorkspaceEditorModule.h"
#include "AnimNextRigVMAssetEditorData.h"


namespace UE::UAF::LayeringEditor
{
class FUAFLayeringEditorModule : public IModuleInterface
{

public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	TSharedRef<SWidget> MakeDocumentWidget(const UE::Workspace::FWorkspaceEditorContext& InContext);
	void GetBreadcrumbTrail(const UE::Workspace::FWorkspaceEditorContext& InContext, TArray<TSharedPtr<UE::Workspace::FWorkspaceBreadcrumb>>& OutBreadcrumbs);
};
}

