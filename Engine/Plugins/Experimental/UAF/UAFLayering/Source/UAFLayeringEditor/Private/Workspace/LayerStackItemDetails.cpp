// Copyright Epic Games, Inc. All Rights Reserved.

#include "LayerStackItemDetails.h"

#include "IWorkspaceEditor.h"
#include "IWorkspaceEditorModule.h"
#include "ToolMenuContext.h"
#include "UAFLayeringStyle.h"
#include "UAFLayerStack.h"
#include "UAFStyle.h"
#include "WorkspaceItemMenuContext.h"
#include "Layers/UAFLayer.h"
#include "Toolkits/AssetEditorToolkitMenuContext.h"
#include "Workspace/UAFLayerStackWorkspaceOutlinerData.h"

namespace UE::UAF::LayeringEditor
{
	bool ULayerStackItemDetails::HandleSelected(const FToolMenuContext& ToolMenuContext) const
	{
		const UWorkspaceItemMenuContext* WorkspaceItemContext = ToolMenuContext.FindContext<UWorkspaceItemMenuContext>();
		const UAssetEditorToolkitMenuContext* AssetEditorContext = ToolMenuContext.FindContext<UAssetEditorToolkitMenuContext>(); 
		if (WorkspaceItemContext && AssetEditorContext)
		{
			if(const TSharedPtr<UE::Workspace::IWorkspaceEditor> WorkspaceEditor = StaticCastSharedPtr<UE::Workspace::IWorkspaceEditor>(AssetEditorContext->Toolkit.Pin()))
			{
				if (WorkspaceItemContext->SelectedExports.Num() > 0)
				{
					const FWorkspaceOutlinerItemExport& SelectedExport = WorkspaceItemContext->SelectedExports[0].GetResolvedExport();
					if (UUAFLayerStack* LayerStack = SelectedExport.GetFirstObjectOfType<UUAFLayerStack>())
					{
						if (const UUAFLayerStack_EditorData* EditorData = UncookedOnly::FUtils::GetEditorData<UUAFLayerStack_EditorData>(LayerStack))
						{
							EditorData->ClearSelectedLayer();
							return true;
						}
					}
				}
			}
		}
		
		return false;
	}

	const FSlateBrush* ULayerStackItemDetails::GetItemIcon(const FWorkspaceOutlinerItemExport& Export) const
	{
		return FUAFLayeringStyle::Get().GetBrush(TEXT("ClassIcon.UAFLayerStack"));
	}

	bool ULayerStackLayerItemDetails::IsExpandedByDefault() const
	{
		return true;
	}

	FString ULayerStackLayerItemDetails::GetDisplayString(const FWorkspaceOutlinerItemExport& Export) const
	{
		const TInstancedStruct<FWorkspaceOutlinerItemData>& Data = Export.GetData();
		if (Data.IsValid() && Data.GetScriptStruct() == FUAFLayerStackLayerOutlinerData::StaticStruct())
		{
			const FUAFLayerStackLayerOutlinerData& LayerData = Data.Get<FUAFLayerStackLayerOutlinerData>();
			return LayerData.LayerName.ToString();
		}
		
		return IWorkspaceOutlinerItemDetails::GetDisplayString(Export);
	}

	const FSlateBrush* ULayerStackLayerItemDetails::GetItemIcon(const FWorkspaceOutlinerItemExport& Export) const
	{
		return FUAFLayeringStyle::Get().GetBrush(TEXT("ClassIcon.UAFLayer"));
	}
	
	bool ULayerStackLayerItemDetails::HandleSelected(const FToolMenuContext& ToolMenuContext) const
	{
		const UWorkspaceItemMenuContext* WorkspaceItemContext = ToolMenuContext.FindContext<UWorkspaceItemMenuContext>();
		const UAssetEditorToolkitMenuContext* AssetEditorContext = ToolMenuContext.FindContext<UAssetEditorToolkitMenuContext>(); 
		if (WorkspaceItemContext && AssetEditorContext)
		{
			if(const TSharedPtr<UE::Workspace::IWorkspaceEditor> WorkspaceEditor = StaticCastSharedPtr<UE::Workspace::IWorkspaceEditor>(AssetEditorContext->Toolkit.Pin()))
			{
				if (WorkspaceItemContext->SelectedExports.Num() > 0)
				{
					const FWorkspaceOutlinerItemExport& SelectedExport = WorkspaceItemContext->SelectedExports[0].GetResolvedExport();
					if (UUAFLayer* SelectedLayer = SelectedExport.GetFirstObjectOfType<UUAFLayer>())
					{
						if (const UUAFLayerStack_EditorData* EditorData = SelectedLayer->GetTypedOuter<UUAFLayerStack_EditorData>())
						{
							EditorData->SelectLayer(SelectedLayer);
							return true;
						}
					}
				}
			}
		}
		
		return false;
	}

	bool ULayerStackLayerItemDetails::HandleDoubleClick(const FToolMenuContext& ToolMenuContext) const
	{
		const UWorkspaceItemMenuContext* WorkspaceItemContext = ToolMenuContext.FindContext<UWorkspaceItemMenuContext>();
		const UAssetEditorToolkitMenuContext* AssetEditorContext = ToolMenuContext.FindContext<UAssetEditorToolkitMenuContext>(); 
		if (WorkspaceItemContext && AssetEditorContext)
		{
			if(const TSharedPtr<UE::Workspace::IWorkspaceEditor> WorkspaceEditor = StaticCastSharedPtr<UE::Workspace::IWorkspaceEditor>(AssetEditorContext->Toolkit.Pin()))
			{
				if (WorkspaceItemContext->SelectedExports.Num() > 0)
				{
					const FWorkspaceOutlinerItemExport& SelectedExport = WorkspaceItemContext->SelectedExports[0].GetResolvedExport();
					if (UUAFLayer* SelectedLayer = SelectedExport.GetFirstObjectOfType<UUAFLayer>())
					{
						if (UUAFLayerStack_EditorData* EditorData = SelectedLayer->GetTypedOuter<UUAFLayerStack_EditorData>())
						{
							if (const UUAFRigVMAsset* RigVMAsset = UncookedOnly::FUtils::GetAsset(EditorData))
							{
								const Workspace::FWorkspaceDocument& Document = WorkspaceEditor->GetFocusedWorkspaceDocument();	
								const FWorkspaceOutlinerItemExport RelativeAssetExport = Document.Export.GetRelativeAssetExport(RigVMAsset);
								const FWorkspaceOutlinerItemExport AssetExport = UncookedOnly::FUtils::MakeAssetReferenceExport(RigVMAsset, RelativeAssetExport);

								WorkspaceEditor->OpenExports({AssetExport});
								EditorData->SelectLayer(SelectedLayer);
								return true;
							}
						}
					}
				}
			}
		}
		
		return false;
	}
}
