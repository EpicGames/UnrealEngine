// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IWorkspaceOutlinerItemDetails.h"

namespace UE::UAF::LayeringEditor
{
	class ULayerStackItemDetails : public UE::Workspace::IWorkspaceOutlinerItemDetails
	{
	public:
		virtual bool HandleSelected(const FToolMenuContext& ToolMenuContext) const override;
		virtual const FSlateBrush* GetItemIcon(const FWorkspaceOutlinerItemExport& Export) const override;
	};
	
	class ULayerStackLayerItemDetails : public UE::Workspace::IWorkspaceOutlinerItemDetails
	{
	public:
		virtual bool IsExpandedByDefault() const override;
		virtual FString GetDisplayString(const FWorkspaceOutlinerItemExport& Export) const override;
		virtual const FSlateBrush* GetItemIcon(const FWorkspaceOutlinerItemExport& Export) const override;
		virtual bool HandleSelected(const FToolMenuContext& ToolMenuContext) const override;
		virtual bool HandleDoubleClick(const FToolMenuContext& ToolMenuContext) const override;
	};
}

