// Copyright Epic Games, Inc. All Rights Reserved.

#include "Workspace/LayerStackViewportController.h"

#include "AdvancedPreviewScene.h"
#include "UAFLayerStack.h"
#include "Component/AnimNextComponent.h"
#include "Engine/PreviewMeshCollection.h"
#include "Module/UAFSystemAssetData.h"
#include "UAF/UAFAssetData.h"
#include "UAF/UAFAssetFactory.h"
#include "UAF/Viewport/ViewportSceneDescription.h"

namespace UE::UAF::LayeringEditor
{
	void FLayerStackViewportController::OnEnter(const FViewportContext& InViewportContext)
	{
		const TObjectPtr<const UUAFLayerStack> LayerStack = Cast<UUAFLayerStack>(InViewportContext.OutlinerObject);
		if (!LayerStack)
		{
			UE_LOGF(LogAnimation, Warning, "Workspace Preview: OutlinerObject was null or not of the expected type: [%ls]", *GetNameSafe(UUAFLayerStack::StaticClass()));
			return;
		}
		
		const TObjectPtr<const UUAFSystem> System = LoadObject<UUAFSystem>(GetTransientPackage(), TEXT("/UAFAnimGraph/Internal/S_SingleGraph.S_SingleGraph"));
		if (!System)
		{
			UE_LOGF(LogAnimation, Warning, "Workspace Preview: Failed to load template system asset");
			return;
		}
		
		if (const UUAFViewportSceneDescription* UAFViewportSceneDescription = Cast<UUAFViewportSceneDescription>(InViewportContext.SceneDescription))
		{
			// Add main mesh to the scene 
			if (UAFViewportSceneDescription->SkeletalMesh)
			{
				AddMeshToPreview(InViewportContext.PreviewScene, System, LayerStack, UAFViewportSceneDescription->SkeletalMesh);
			}

			// Add any additional meshes to the scene 
			if (UAFViewportSceneDescription->AdditionalMeshes)
			{
				for (int32 MeshIndex = 0; MeshIndex < UAFViewportSceneDescription->AdditionalMeshes->SkeletalMeshes.Num(); ++MeshIndex)
				{
					const TSoftObjectPtr<USkeletalMesh> SkelMeshSoftPtr = UAFViewportSceneDescription->AdditionalMeshes->SkeletalMeshes[MeshIndex].SkeletalMesh;
				
					if (!SkelMeshSoftPtr.IsNull())
					{
						if (USkeletalMesh* SkeletalMesh = SkelMeshSoftPtr.LoadSynchronous())
						{
							AddMeshToPreview(InViewportContext.PreviewScene, System, LayerStack, SkeletalMesh);
						}
					}
				}
			}
		}
	}
	
	void FLayerStackViewportController::AddMeshToPreview(FAdvancedPreviewScene* PreviewScene, const TObjectPtr<const UUAFSystem> System, const TObjectPtr<const UUAFLayerStack> LayerStack, USkeletalMesh* InSkeletalMesh)
	{
		if (!PreviewScene || !InSkeletalMesh || !System || !LayerStack)
		{
			return;
		}
		
		if (const TObjectPtr<UWorld> PreviewWorld = PreviewScene->GetWorld())
		{
			if (AActor* PreviewActor = PreviewWorld->SpawnActor<AActor>(AActor::StaticClass()))
			{
				PreviewActors.Add(PreviewActor);
				
				// Create SkeletalMeshComponent for actor
				if (USkeletalMeshComponent* SkeletalMeshComponent = NewObject<USkeletalMeshComponent>(PreviewActor))
				{
					PreviewScene->AddComponent(SkeletalMeshComponent, FTransform::Identity);
					
					PreviewActor->SetRootComponent(SkeletalMeshComponent);
					SkeletalMeshComponent->SetEnableAnimation(false);
					SkeletalMeshComponent->SetSkeletalMesh(InSkeletalMesh);
				}
				else
				{
					UE_LOGF(LogAnimation, Warning, "Workspace Preview: Failed to create a valid SkeletalMeshComponent");
				}

				// Create UAFComponent for actor
				if (UUAFComponent* UAFComponent = NewObject<UUAFComponent>(PreviewActor))
				{
					PreviewActor->AddInstanceComponent(UAFComponent);
					
					TInstancedStruct<FUAFSystemFactoryAsset> AssetDataFromFactory = UE::UAF::FAssetDataFactory::CreateUAFAssetDataFromObject<FUAFSystemFactoryAsset>(System);
					UAFComponent->SetAsset(MoveTemp(AssetDataFromFactory));
					UAFComponent->RegisterComponent();
					UAFComponent->InitializeComponent();
					
					const bool bSet = UAFComponent->SetVariable<const TObjectPtr<const UObject>>(FAnimNextVariableReference::FromName("Graph", System), LayerStack);
					if (!bSet)
					{
						UE_LOGF(LogAnimation, Warning, "Workspace Preview: Could not set %ls as the graph asset in %ls", *GetNameSafe(LayerStack), *GetNameSafe(System));
					}
				}
				else
				{
					UE_LOGF(LogAnimation, Warning, "Workspace Preview: Failed to create a valid UAFComponent");
				}
			}
			else
			{
				UE_LOGF(LogAnimation, Warning, "Workspace Preview: Failed to spawn preview actor");
			}
		}
	}
	
	void FLayerStackViewportController::OnExit(FAdvancedPreviewScene* PreviewScene)
	{
		for (AActor* PreviewActor : PreviewActors)
		{
			if (PreviewActor)
			{
				if (PreviewScene && PreviewActor->GetRootComponent())
				{
					PreviewScene->RemoveComponent(PreviewActor->GetRootComponent());
				}
			
				PreviewActor->Destroy();
				PreviewActor = nullptr;
			}
		}
		
		PreviewActors.Empty();
	}
}
