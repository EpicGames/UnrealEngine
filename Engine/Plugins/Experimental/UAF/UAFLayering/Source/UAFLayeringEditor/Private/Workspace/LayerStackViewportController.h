// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UAFLayerStack.h"
#include "WorkspaceViewportController.h"
#include "Engine/SkeletalMesh.h"
#include "Module/AnimNextModule.h"

namespace UE::UAF::LayeringEditor
{
	class FLayerStackViewportController : public Workspace::IWorkspaceViewportController
	{
	public:
		virtual void OnEnter(const FViewportContext& InViewportContext) override;
		virtual void OnExit(FAdvancedPreviewScene* PreviewScene) override;

	private:
		void AddMeshToPreview(FAdvancedPreviewScene* PreviewScene, const TObjectPtr<const UUAFSystem> System, const TObjectPtr<const UUAFLayerStack> LayerStack, USkeletalMesh* InSkeletalMesh);
		
		TArray<AActor*> PreviewActors;
	};
}
