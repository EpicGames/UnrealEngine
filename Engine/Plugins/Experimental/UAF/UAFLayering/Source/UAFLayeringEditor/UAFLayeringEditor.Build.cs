// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using UnrealBuildTool.Rules;

public class UAFLayeringEditor : ModuleRules
{
	public UAFLayeringEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
				"WorkspaceEditor",
				"RigVMDeveloper",
				"UAFUncookedOnly",
				"InputCore"
	
				// ... add other public dependencies that you statically link with here ...
			}
		);
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"UAFAnimGraph",
				"UnrealEd",
				"AssetDefinition",
				"UAFLayering",
				"UAF",
				"RigVM",
				"EditorInteractiveToolsFramework",
				"InteractiveToolsFramework",
				"EditorFramework",
				"UAFEditor",
				"UAFLayeringUncookedOnly"


				// ... add private dependencies that you statically link with here ...	
			}
			);

		

		PrivateIncludePathModuleNames.AddRange(
			new string[]
			{
				"UAFEditor",
			}
		);
	}
}
