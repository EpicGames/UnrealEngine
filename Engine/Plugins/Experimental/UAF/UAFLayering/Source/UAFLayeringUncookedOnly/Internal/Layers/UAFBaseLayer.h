// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAFLayer.h"
#include "UAFBaseLayer.generated.h"

// The editor representation of a base layer in a layer stack
UCLASS(editinlinenew, meta =(Hidden))
class UAFLAYERINGUNCOOKEDONLY_API UUAFBaseLayer : public UUAFLayer
{
	GENERATED_BODY()

public:
	UUAFBaseLayer();
	virtual TSharedRef<SWidget> CreateLayerWidget() override;
};


