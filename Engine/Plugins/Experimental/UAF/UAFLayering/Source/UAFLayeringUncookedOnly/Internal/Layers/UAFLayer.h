// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CoreMinimal.h"
#include "LayeringUncookedOnlyTypes.h"
#include "Layers/UAFLayerBlendProviderBase.h"
#include "Layers/UAFLayerContentProviderBase.h"
#include "UAFLayerStack_EditorData.h"
#include "UAFLayer.generated.h"

class UAnimNextController;

enum class EAnimNextEditorDataNotifType : uint8;


// The editor representation of a layer in a layer stack
UCLASS(editinlinenew)
class UAFLAYERINGUNCOOKEDONLY_API UUAFLayer : public UObject
{
	GENERATED_BODY()

public:
	UUAFLayer();
	
	// Creates the widget to represent this layer within the layer stack 
	virtual TSharedRef<SWidget> CreateLayerWidget();
	
	// Responsible for creating all traits that describe this layer 
	// Per default this will request the LayerContentProvider and LayerBlendProvider to specify the exact implementation 
	// @return Returns the output pin of the created layer. This can be nullptr if the creation failed. 
	virtual URigVMPin* CreateLayerGraphContentTraits(UE::UAF::Layering::FLayerCreationContext& LayerCreationContext);
	
	// Toggles the preview state of this layer 
	void TogglePreviewVisibility();
	
	// Gets the current layer name 
	FName GetLayerName() const;
	
	// Renames this layer 
	void RenameLayer(const FName NewName);
	
	// Checks if the passed in name is a valid name for this layer 
	bool IsLayerNameValid(const FName NewName) const;
	
	EUAFLayerState GetLayerState() const;
	void SetLayerState(const EUAFLayerState InLayerState);
	
	bool IsLayerBlendProviderValid() const;
	TInstancedStruct<FUAFLayerBlendProviderBase>& GetLayerBlendProvider();
	void SetLayerBlendProvider(const TInstancedStruct<FUAFLayerBlendProviderBase>& InLayerBlendProvider);
	
	bool IsLayerContentProviderValid() const;
	TInstancedStruct<FUAFLayerContentProviderBase>& GetLayerContentProvider();
	void SetLayerContentProvider(const TInstancedStruct<FUAFLayerContentProviderBase>& InLayerContentProvider);
	
	void BroadcastModified(EAnimNextEditorDataNotifType InType, bool bNotifyLayerLayoutChanges = false) const;

#if WITH_EDITOR
	virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;
	virtual void GetObjectReferences(TArray<const UObject*>& OutReferencedObjects) const;
#endif
	
protected:
	// The current state of the layer. 
	UPROPERTY(EditAnywhere, Category = "Layer")
	EUAFLayerState LayerState = EUAFLayerState::Enabled;

	// The name of this layer. It has to be unique within one layer stack asset. 
	UPROPERTY(EditAnywhere, Category = "Layer")
	FName LayerName = NAME_None;
	
	UPROPERTY(EditAnywhere, Category = "Layer", meta = (ExcludeBaseStruct))
	TInstancedStruct<FUAFLayerContentProviderBase> LayerContentProvider;
	
	UPROPERTY(EditAnywhere, Category = "Layer", meta = (ExcludeBaseStruct))
	TInstancedStruct<FUAFLayerBlendProviderBase> LayerBlendProvider;
	
	FName CachedLayerOutputPoseName = NAME_None;
	
	FName PreRenameLayerName = NAME_None;
};

