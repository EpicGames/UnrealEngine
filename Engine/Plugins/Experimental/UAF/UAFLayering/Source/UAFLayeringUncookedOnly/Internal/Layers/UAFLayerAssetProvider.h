// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAFLayerContentProviderBase.h"
#include "Factory/AnimNextFactoryParams.h"
#include "UAF/UAFAssetData.h"

#include "UAFLayerAssetProvider.generated.h"

class UUAFLayer;
class SWidget;

// Layer content provider to support any pose outputting asset 
USTRUCT()
struct FUAFLayerAssetProvider : public FUAFLayerContentProviderBase
{
	GENERATED_BODY()

public: 
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Note: We need to explicitly disable warnings on these constructors/operators for clang to be happy with deprecated variables
	// this is a requirement for clang to compile without warnings.
	FUAFLayerAssetProvider() = default;
	virtual ~FUAFLayerAssetProvider() override = default;
	FUAFLayerAssetProvider(const FUAFLayerAssetProvider&) = default;
	FUAFLayerAssetProvider(FUAFLayerAssetProvider&&) = default;
	FUAFLayerAssetProvider& operator=(const FUAFLayerAssetProvider&) = default;
	FUAFLayerAssetProvider& operator=(FUAFLayerAssetProvider&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	virtual URigVMPin* CreateLayerContentTrait(UE::UAF::Layering::FLayerCreationContext& LayerCreationContext) override;
	virtual TSharedRef<SWidget> CreateLayerContentWidget(UUAFLayer* InLayer) override;

	void SetLayerAsset(const UObject* InNewAsset);
	
#if WITH_EDITOR
	virtual void GetObjectReferences(TArray<const UObject*>& OutReferencedObjects) const override;
#endif
	
#if WITH_EDITORONLY_DATA
	void PostSerialize(const FArchive& Ar);
#endif
	
private:
	void OnLayerAssetSelectionChanged(const FAssetData& InAssetData);
	
private:
	UPROPERTY(EditAnywhere, Category = "Layer|Content")
	TInstancedStruct<FUAFGraphFactoryAsset> AssetData; 
	
	UPROPERTY()
	TWeakObjectPtr<UUAFLayer> OuterLayer = nullptr;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.8, "Factory Parameters are not used directly anymore, instead it uses FUAFAssetData internally")
	UPROPERTY()
	FAnimNextFactoryParams Parameters_DEPRECATED;
	
	UE_DEPRECATED(5.8, "Using UObject based assets directly is deprecated, use AssetData instead")
	UPROPERTY()
	TObjectPtr<UObject> LayerAsset_DEPRECATED = nullptr;
#endif
	
};

template<>
struct TStructOpsTypeTraits<FUAFLayerAssetProvider> : public TStructOpsTypeTraitsBase2<FUAFLayerAssetProvider>
{
#if WITH_EDITORONLY_DATA
	enum
	{
		WithPostSerialize = true,
	};
#endif
};