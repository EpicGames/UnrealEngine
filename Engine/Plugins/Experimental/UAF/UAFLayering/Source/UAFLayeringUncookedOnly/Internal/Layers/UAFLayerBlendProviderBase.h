// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CoreMinimal.h"
#include "LayeringUncookedOnlyTypes.h"
#include "Widgets/SNullWidget.h"
#include "UAFLayerBlendProviderBase.generated.h"

class URigVMPin;
class UUAFLayer;
class SWidget;

USTRUCT(meta=(Hidden))
struct FUAFLayerBlendProviderBase
{
	GENERATED_BODY()

public:
	FUAFLayerBlendProviderBase() = default;
	virtual ~FUAFLayerBlendProviderBase() = default;
	
	// Responsible for creating traits to represent this blend and link inputs to the relevant traits.
	// @return The final output pin of this layer blend which can be linked as input by other layers or be linked to the result of the graph
	virtual URigVMPin* CreateBlendGraphTrait(UE::UAF::Layering::FLayerCreationContext& LayerCreationContext) { return nullptr; };
	
	// Creates the widget to represent this blend in the layer stack editor 
	virtual TSharedRef<SWidget> CreateLayerBlendWidget(UUAFLayer* OuterLayer) { return SNullWidget::NullWidget;} ;
	
	// Optional override of the containing layers background brush 
	virtual const FSlateBrush* GetOverrideLayerBackground() const { return nullptr; };
	
	// Optional override of the containing layers indicator color
	virtual bool GetOverrideIndicatorColor(FSlateColor& OutSlateColor) const { return false; };
	
	void SetAlwaysUpdateChildren(bool bInAlwaysUpdate) { bAlwaysUpdateChildren = bInAlwaysUpdate; };
	bool GetAlwaysUpdateChildren() const { return bAlwaysUpdateChildren; };
	
#if WITH_EDITOR
	virtual void GetObjectReferences(TArray<const UObject*>& OutReferencedObjects) const {};
#endif
	
private:
	bool bAlwaysUpdateChildren = false;	
};


