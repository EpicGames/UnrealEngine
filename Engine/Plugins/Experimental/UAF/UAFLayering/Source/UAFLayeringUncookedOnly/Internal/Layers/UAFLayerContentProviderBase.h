// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SNullWidget.h"
#include "LayeringUncookedOnlyTypes.h"
#include "UAFLayerContentProviderBase.generated.h"

class URigVMPin;
class SWidget;
class UUAFLayer;

USTRUCT(meta=(Hidden))
struct FUAFLayerContentProviderBase 
{
	GENERATED_BODY()
public:		
	FUAFLayerContentProviderBase() = default;
	virtual ~FUAFLayerContentProviderBase() = default;

	// Responsible for creating the traits to play the content of this layer 
	// @return The output pin of this layers content trait stack which can be linked to the blend trait stack of this layer 
	virtual URigVMPin* CreateLayerContentTrait(UE::UAF::Layering::FLayerCreationContext& LayerCreationContext)  { return nullptr; };
	
	// Responsible for creating the widget to represent this layers content in the layer stack editor 
	virtual TSharedRef<SWidget> CreateLayerContentWidget(UUAFLayer* InLayer) { return SNullWidget::NullWidget; };  
	
#if WITH_EDITOR
	virtual void GetObjectReferences(TArray<const UObject*>& OutReferencedObjects) const {};
#endif
	
};

