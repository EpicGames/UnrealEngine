// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CoreMinimal.h"
#include "UAFLayerBlendProviderBase.h"
#include "IStructureDataProvider.h"
#include "UAFLayeringTypes.h"
#include "StructUtils/InstancedStruct.h"

#include "UAFLayerDefaultBlendProvider.generated.h"

class UAnimNextController;
class URigVMPin;
class UUAFBlendMask;
class UUAFBlendProfile;
class UUAFLayer;
class FUAFDefaultLayerBlendStructureDataProvider;

USTRUCT()
struct FUAFDefaultBlendProvider : public FUAFLayerBlendProviderBase
{
	GENERATED_BODY()

public:
	// Begin FUAFLayerBlendProvider
	virtual URigVMPin* CreateBlendGraphTrait(UE::UAF::Layering::FLayerCreationContext& LayerCreationContext) override;
	virtual TSharedRef<SWidget> CreateLayerBlendWidget(UUAFLayer* InLayer) override;
	
	// TODO: We can abstract this out to some layer override struct or to delegates we can pass into the widget 
	virtual const FSlateBrush* GetOverrideLayerBackground() const override;
	virtual bool GetOverrideIndicatorColor(FSlateColor& OutSlateColor) const override;
	// End FUAFLayerBlendProvider

public:
	// TODO: This should be a binding
	// If the layer is currently enabled at all 
	UPROPERTY(EditAnywhere, Category = "Layer|BlendSettings")
	bool bLayerEnabled = true;
	
	// The blend mode this layer uses to blend its result with the pose result of the previous layer 
	UPROPERTY(EditAnywhere, Category = "Layer")
	EUAFLayerBlendMode BlendMode = EUAFLayerBlendMode::Blend;
	
	// If not custom curve is used, which easing method should be used when this layer is blending in or out  
	UPROPERTY(EditAnywhere, Category = "Layer|BlendSettings", meta = (EditCondition = "BlendCurve == nullptr", EditConditionHides))
	EAlphaBlendOption BlendOption = EAlphaBlendOption::Linear;
	
	// TODO: This should be a binding
	// The weight to use when blending this layer in. 0 == this layer won't get blended in at all, 1 == 100% of the layer gets blended in 
	UPROPERTY(EditAnywhere, Category = "Layer", meta=(ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0", Delta=0.1))
	float LayerWeight = 1.0f;
	
	// TODO: This should be a binding
	// How long it should take to blend this layer in if it has been disabled previously 
	UPROPERTY(EditAnywhere, Category = "Layer|BlendSettings", meta=(Delta=0.1))
	float LayerBlendInTime = 0.5f;

	// TODO: This should be a binding
	// How long it should take to blend this layer out 
	UPROPERTY(EditAnywhere, Category = "Layer|BlendSettings", meta=(Delta=0.1))
	float LayerBlendOutTime = 0.5f;
	
	// TODO: This should be a binding
	// An optional blend mask to apply when blending this layers pose result with the previous layer result 
	UPROPERTY(EditAnywhere, Category = "Layer")
	TObjectPtr<UUAFBlendMask> BlendMask = nullptr;
	
	// TODO: This should be a binding
	// The custom blend curve to use when blending this layer in or out 
	UPROPERTY(EditAnywhere, Category = "Layer|BlendSettings")
	TObjectPtr<class UCurveFloat> BlendCurve = nullptr;
	
	// TODO: This should be a binding
	// The blend profile to use to add more fine grain control over how certain bones should blend in or out 
	// This can be used additionally to the blend mask that is used to apply this layer 
	UPROPERTY(EditAnywhere, Category = "Layer|BlendSettings")
	TObjectPtr<UUAFBlendProfile> BlendProfile = nullptr;
	
	// TODO: In the future we should be able to read this out of the blend mask 
	// If a blend mask is applied, this is the named set for abstract hierarchies that mask relates to
	UPROPERTY(EditAnywhere, Category = "Layer", meta=(EditCondition="BlendMask != nullptr"))
	FName SetName = NAME_None;
};

namespace UE::UAF::Layering
{
class FDefaultLayerBlendStructureDataProvider : public IStructureDataProvider
{
public:
	FDefaultLayerBlendStructureDataProvider(TInstancedStruct<FUAFLayerBlendProviderBase>* InBlendProvider)
		: BlendProvider(InBlendProvider)
	{}

	virtual bool IsValid() const override;
	virtual const UStruct* GetBaseStructure() const override;
	virtual void GetInstances(TArray<TSharedPtr<FStructOnScope>>& OutInstances, const UStruct* ExpectedBaseStructure) const override;

protected:
	TInstancedStruct<FUAFLayerBlendProviderBase>* BlendProvider = nullptr;
};
}



