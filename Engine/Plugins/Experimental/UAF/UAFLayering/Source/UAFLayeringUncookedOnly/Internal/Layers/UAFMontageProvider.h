// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAFLayerContentProviderBase.h"

#include "UAFMontageProvider.generated.h"

class UUAFLayer;
class SWidget;

// Layer content provider to support legacy montages
USTRUCT(DisplayName="Montage Layer")
struct FUAFMontageProvider : public FUAFLayerContentProviderBase
{
	GENERATED_BODY()

public:
	virtual URigVMPin* CreateLayerContentTrait(UE::UAF::Layering::FLayerCreationContext& LayerCreationContext) override;
	virtual TSharedRef<SWidget> CreateLayerContentWidget(UUAFLayer* InLayer) override;
	
private:
	// The name of this slot 
	UPROPERTY(EditAnywhere, Category = "Layer|Content")
	FName SlotName = NAME_None;
	
	// If true it will always update the source input, even when montages are fully blended in 
	UPROPERTY(EditAnywhere, Category = "Layer|Content")
	bool bAlwaysUpdateSource = false;
	
	// Disables synchronization for this slot even if the playing montage has a sync group setup 
	UPROPERTY(EditAnywhere, Category = "Layer|Content")
	bool bDisableSynchronization = false;
	
	// If true, automatically disables and enables the layer based on if a montage in the given slot is active
	// This will cause the content of this layer to always update even if the weight is 0 to auto enable itself 
	UPROPERTY(EditAnywhere, Category = "Layer|Content")
	bool bAutoEnableLayerWithMontage = false;
	
	// If true, automatically sets the layer blend in and blend out time based on the montage playing 
	UPROPERTY(EditAnywhere, Category = "Layer|Content")
	bool bAutoSetBlendTimesFromMontage = false;
	
};

