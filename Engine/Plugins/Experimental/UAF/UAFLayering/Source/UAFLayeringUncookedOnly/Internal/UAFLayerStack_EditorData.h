// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorUndoClient.h"
#include "LayeringUncookedOnlyTypes.h"
#include "Graph/AnimNextAnimationGraph_EditorData.h"
#include "UAFLayerStack_EditorData.generated.h"

class UUAFLayer;
class UUAFBaseLayer;
class UUAFLayerStack;

#define UE_API UAFLAYERINGUNCOOKEDONLY_API

DECLARE_MULTICAST_DELEGATE(FLayerLayoutChangedDelegate);
DECLARE_MULTICAST_DELEGATE_OneParam(FLayerSelectionChangedDelegate, const TWeakObjectPtr<UUAFLayer>);

namespace UE::UAF::Layering
{
	enum class EBaseLayerInclusion
	{
		// Includes the base layer when indexing in the layer stack
		Include,
		// Excludes the base layer when indexing in the layer stack
		Exclude
	};
}

UCLASS()
class UUAFLayerStack_EditorData : public UUAFAnimGraph_EditorData, public FEditorUndoClient
{
	GENERATED_BODY()

public:
	UE_API UUAFLayerStack_EditorData();
	UE_API virtual ~UUAFLayerStack_EditorData() override;
	
	// Begin FEditorUndoClient interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	// End FEditorUndoClient interface
	
	// Gets the number of total layers 
	// Per default, this also includes the base layer
	int32 GetNumLayers(const UE::UAF::Layering::EBaseLayerInclusion IncludeBaseLayer = UE::UAF::Layering::EBaseLayerInclusion::Include) const;
	
	// Gets the index for the given layer, either including the base layer or excluding it
	int32 GetIndexForLayer(const TObjectPtr<UUAFLayer> InLayer, const UE::UAF::Layering::EBaseLayerInclusion IncludeBaseLayer = UE::UAF::Layering::EBaseLayerInclusion::Include) const;
	
	// Gets all layers including the base layer 
	TArray<TObjectPtr<UUAFLayer>> GetAllLayers() const;
	
	// Checks if the given layer is the base layer of this layer stack
	bool IsBaseLayer(const TObjectPtr<UUAFLayer> InLayer) const;
	
	// Checks if the given layer is the last layer in this layer stack
	bool IsLastLayer(const TObjectPtr<UUAFLayer> InLayer) const;
	
	// Moves the given layer to a specific index in the layer stack
	// Does not affect the base layer of the stack 
	// NewLayerIndex == 0 is equivalent to the first layer under the base layer
	void MoveLayerToIndex(const TObjectPtr<UUAFLayer> LayerToMove, int32 NewLayerIndex);

	// Decreases the index of the given layer within the layers array of this layer stack and moves it visually up in the stack
	// If index == 0, this layer will be the next layer under the base layer
	// Does not affect the base layer of the stack 
	void MoveLayerUp(const TObjectPtr<UUAFLayer> LayerToMove);
	
	// Increases the index of the given layer within the layers array of this layer stack and moves it visually down in the stack
	// Does not affect the base layer of the stack
	void MoveLayerDown(const TObjectPtr<UUAFLayer> LayerToMove);
	
	// Removes the given layer from the layer stack 
	void RemoveLayer(const TObjectPtr<UUAFLayer> LayerToDelete);
	
	// Checks if NewLayerName is a valid name for the given layer 
	bool IsLayerNameValid(const TObjectPtr<const UUAFLayer> InLayer, const FName NewLayerName) const;
	
	// Generates a unique name for the given layer in the current layer stack
	// Unless the layer name is already valid, then it will return that 
	FName GetUniqueNameForLayer(const TObjectPtr<UUAFLayer> InLayer) const;
	
	// Set the state of the given layer
	void SetLayerState(const TObjectPtr<UUAFLayer> Layer, EUAFLayerState LayerState);
	
	// Selects the given layer within the layer stack 
	UE_API void SelectLayer(const TObjectPtr<UUAFLayer> Layer) const;

	// Clears the current selection of the layer stack
	UE_API void ClearSelectedLayer() const;
	
	// Creates a new layer with a default asset and blend provider and adds it on top of the stack
	TObjectPtr<UUAFLayer> AddDefaultAssetBasedLayer(const FAssetData& InAsset);

protected:
	// Begin UUAFAnimGraph_EditorData interface
	UE_API virtual void Initialize(bool bRecompileVM) override;
	UE_API virtual TSubclassOf<UAssetUserData> GetAssetUserDataClass() const override;
	UE_API virtual TConstArrayView<TSubclassOf<UUAFRigVMAssetEntry>> GetEntryClasses() const override;
	virtual void OnPreCompileGetProgrammaticGraphs(const FRigVMCompileSettings& InSettings, FAnimNextGetGraphCompileContext& OutCompileContext) override;
	// End UUAFAnimGraph_EditorData interface
	
	// Begin UObject interface
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
	// End UObject interface

private:
	// Broadcasts if anything has changed in the current layer stack, with the option to refresh the layering UI
	void BroadcastLayerStackChanged(EAnimNextEditorDataNotifType InType, bool bRefreshUI = false);
	
	// Notify the layering UI to refresh its state
	void NotifyLayerLayoutChanged() const;
	
	void OnLayerAdded(const TObjectPtr<UUAFLayer> NewLayer);
	
public:
	UPROPERTY(Transient)
	TWeakObjectPtr<URigVMGraph> CreatedGraph = nullptr;
	
	FLayerLayoutChangedDelegate OnLayerLayoutChanged;
	FLayerSelectionChangedDelegate OnLayerSelectionChanged;

private:
	UPROPERTY(Instanced, VisibleAnywhere, Category = "LayerStack")
	TObjectPtr<UUAFBaseLayer> BaseLayer;

	UPROPERTY(Instanced, EditAnywhere, Category = "LayerStack")
	TArray<TObjectPtr<UUAFLayer>> Layers;
	
	friend class UUAFLayerStackFactory;
	friend class UUAFLayer;
};

#undef UE_API
