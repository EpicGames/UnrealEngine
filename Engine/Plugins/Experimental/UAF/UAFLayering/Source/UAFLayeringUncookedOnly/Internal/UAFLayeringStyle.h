// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class FUAFLayeringStyle final : public FSlateStyleSet
{
public:
	static UAFLAYERINGUNCOOKEDONLY_API FUAFLayeringStyle& Get();

private:
	FUAFLayeringStyle();
	~FUAFLayeringStyle();
};