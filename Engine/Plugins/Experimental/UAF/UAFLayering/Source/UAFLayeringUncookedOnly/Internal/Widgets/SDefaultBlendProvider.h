// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISinglePropertyView.h"
#include "Misc/NotifyHook.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::UAF::Layering
{
	class FDefaultLayerBlendStructureDataProvider;
}

class ISinglePropertyView;
class UUAFLayer;

namespace UE::UAF::Layering
{
class SDefaultBlendProvider : public SCompoundWidget, public FNotifyHook
{
public:
	SLATE_BEGIN_ARGS(SDefaultBlendProvider) {}
		SLATE_ARGUMENT(UUAFLayer*, Layer)
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);
	
	virtual void NotifyPreChange( class FEditPropertyChain* PropertyAboutToChange ) override;
	virtual void NotifyPostChange( const FPropertyChangedEvent& PropertyChangedEvent, class FEditPropertyChain* PropertyThatChanged ) override;

private:
	TWeakObjectPtr<UUAFLayer> WeakLayer = nullptr;
};

}
