// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"

class UUAFLayer;

namespace UE::UAF::Layering
{
class SLayer : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SLayer) {}
		SLATE_ARGUMENT(UUAFLayer*, Layer)
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs);

private: 
	const FSlateBrush* GetVisibilityBrushForLayer() const;
	const FSlateBrush* GetAdvancedSettingBrushForLayer() const;
	const FSlateBrush* GetBackgroundBrush() const;
	const FSlateBrush* GetBackgroundOverlayBrush() const;
	
	FSlateColor GetIndicatorColor() const;
	FText GetLayerName() const;
	bool IsSelected() const;
	bool VerifyLayerName(const FText& Text, FText& OutErrorMessage) const;
	void OnLayerNameCommitted(const FText& NewText, ETextCommit::Type CommitInfo) const;

private: 
	TWeakObjectPtr<UUAFLayer> Layer = nullptr;
	bool bAdvancedSettingsExpanded = false;
};
	
}
