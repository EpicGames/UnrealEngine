// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "IWorkspaceEditorModule.h"
#include "UAFLayerStack_EditorData.h"
#include "Widgets/Views/SListView.h"

class UUAFLayer;
class FUICommandList;
class UUAFLayerStack;

namespace UE::Workspace
{
	class IWorkspaceEditor;
}

namespace UE::UAF::Layering
{
class SLayerStack : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SLayerStack) {}
		SLATE_ARGUMENT(UUAFLayerStack*, LayerStack)
		SLATE_ARGUMENT(TSharedPtr<UE::Workspace::IWorkspaceEditor>, WorkspaceEditor)
		SLATE_EVENT(FSimpleDelegate, OnSelectedLayerChanged)
	SLATE_END_ARGS()
	
	UAFLAYERINGUNCOOKEDONLY_API void Construct(const FArguments& InArgs);
	UAFLAYERINGUNCOOKEDONLY_API virtual ~SLayerStack() override;
	
private:
	TSharedRef<ITableRow> GenerateLayerRow(TWeakObjectPtr<UUAFLayer> Item, const TSharedRef<STableViewBase>& OwnerTable) const;
	void OnListSelectionChanged(TWeakObjectPtr<UUAFLayer> SelectedItem, ESelectInfo::Type SelectInfo) const;
	TSharedPtr<SWidget> OnConstructContextMenu() const;
	void OnLayerLayoutChanged();
	void OnLayerSelectionChanged(const TWeakObjectPtr<UUAFLayer> SelectedLayer) const;
	void CacheLayersForList();
private:
	TSharedPtr<SListView<TWeakObjectPtr<UUAFLayer>>> LayerListView = nullptr;
	TArray<TWeakObjectPtr<UUAFLayer>> CachedLayerPointers;
	
	TSharedPtr<FUICommandList> CommandList = nullptr;
	TWeakObjectPtr<UUAFLayerStack> LayerStack = nullptr;
	TWeakPtr<UE::Workspace::IWorkspaceEditor> WeakWorkspaceEditor = nullptr;
};

}
