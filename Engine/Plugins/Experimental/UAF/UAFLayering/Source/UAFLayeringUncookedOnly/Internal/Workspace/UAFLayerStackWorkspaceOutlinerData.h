// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "AnimNextAssetWorkspaceAssetUserData.h"
#include "UAFLayerStackWorkspaceOutlinerData.generated.h"

USTRUCT()
struct FUAFLayerStackWorkspaceOutlinerData : public FAnimNextRigVMAssetOutlinerData
{
	GENERATED_BODY()
	
	FUAFLayerStackWorkspaceOutlinerData() = default;
};

USTRUCT()
struct FUAFLayerStackLayerOutlinerData : public FWorkspaceOutlinerItemData
{
	GENERATED_BODY()
	
	FUAFLayerStackLayerOutlinerData() = default;

	UPROPERTY()
	FName LayerName = NAME_None;
};
