// Copyright Epic Games, Inc. All Rights Reserved.


#include "Layers/UAFBaseLayer.h"

#include "UAFLayeringStyle.h"
#include "Components/HorizontalBox.h"
#include "Layers/UAFLayerAssetProvider.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FUAFLayeringEditorModule"

UUAFBaseLayer::UUAFBaseLayer()
{
	SetLayerContentProvider(TInstancedStruct<FUAFLayerAssetProvider>::Make());
	LayerBlendProvider.Reset();
}

TSharedRef<SWidget> UUAFBaseLayer::CreateLayerWidget()
{
	return SNew(SOverlay)
		+ SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			SNew(SBorder)
				.BorderImage(FUAFLayeringStyle::Get().GetBrush(TEXT("Layer.Background")))
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(5.0f, 10.f)
						[
							SNew(SVerticalBox)
								+ SVerticalBox::Slot()
								.AutoHeight()
								.Padding(10.0f, 0.0f)
								[
									SNew(STextBlock)
									.Text_Lambda([this]() -> FText
									{
										return FText::FromName(LayerName);
									})
								]
								+ SVerticalBox::Slot()
								.AutoHeight()
								.Padding(10.0f, 5.0f)
								[
									LayerContentProvider.IsValid() ? LayerContentProvider.GetMutable().CreateLayerContentWidget(this) : SNullWidget::NullWidget
								]
						]
					+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SVerticalBox)
							+SVerticalBox::Slot()
							.Padding(20.0f, 30.0f, 10.0f, 5.0f)
							.AutoHeight()
								[
									// Base Layer Cached Pose
									SNew(STextBlock)
										.Text(FText::FromString(TEXT("Base Cached Pose")))
								]
							+SVerticalBox::Slot()
							.Padding(20.0f, 2.0f)
							.AutoHeight()
								[
									SNew(SHorizontalBox)
									+SHorizontalBox::Slot()
									.Padding(0.0f, 2.0f)
									.AutoWidth()
									[
										SNew(SImage)
										.Image(FUAFLayeringStyle::Get().GetBrush(TEXT("Layer.PoseIconSmall")))
									]
									+SHorizontalBox::Slot()
									.Padding(5.0f, 0.0f)
									[
										SNew(SEditableTextBox)
												.Text(FText::FromString(TEXT("Base Pose Binding")))
												.IsEnabled(false)
									]
								]
						]
				]
		];
}

#undef LOCTEXT_NAMESPACE
