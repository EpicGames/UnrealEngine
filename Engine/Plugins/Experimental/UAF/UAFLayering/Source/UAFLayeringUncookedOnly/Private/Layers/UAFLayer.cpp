// Copyright Epic Games, Inc. All Rights Reserved.


#include "Layers/UAFLayer.h"

#include "UAFLayerStack_EditorData.h"
#include "Layers/UAFLayerDefaultBlendProvider.h"
#include "Layers/UAFLayerAssetProvider.h"
#include "Widgets/SLayer.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "FUAFLayer"

UUAFLayer::UUAFLayer()
{
	SetLayerContentProvider(TInstancedStruct<FUAFLayerAssetProvider>::Make());
	SetLayerBlendProvider(TInstancedStruct<FUAFDefaultBlendProvider>::Make());
}

URigVMPin* UUAFLayer::CreateLayerGraphContentTraits(UE::UAF::Layering::FLayerCreationContext& LayerCreationContext)
{
	LayerCreationContext.Layer = this;
	
	// Create layer content traits 
	URigVMPin* ContentOutputPin = LayerContentProvider.IsValid() ? LayerContentProvider.GetMutable().CreateLayerContentTrait(LayerCreationContext) : nullptr;
	if (ContentOutputPin == nullptr)
	{
		LayerCreationContext.CompileSettings.ReportError(TEXT("Layer content traits could not be created successfully!"));
		return nullptr;
	}
	
	// The blend trait can be optional, some layers e.g. the base layer might only provide content but no blend 
	// in these cases we pass through the layer content output 
	if (LayerBlendProvider.IsValid())
	{
		// The layer content is the second input for the layering trait (ChildB) 
		// The previous layer output will be at index 0, if applicable 
		LayerCreationContext.LayerInputs[1] = ContentOutputPin;
		
		if (URigVMPin* BlendOutputPin = LayerBlendProvider.GetMutable().CreateBlendGraphTrait(LayerCreationContext))
		{
			return BlendOutputPin;
		}
	}
	
	return ContentOutputPin;
}

void UUAFLayer::TogglePreviewVisibility()
{
	switch (LayerState)
	{
	case EUAFLayerState::Enabled:
		LayerState = EUAFLayerState::PreviewDisabled;
		break;
	case EUAFLayerState::PreviewDisabled:
		LayerState = EUAFLayerState::Enabled;
		break;
	case EUAFLayerState::Disabled:
	default:
		// Do nothing 
		break;
	}
}

void UUAFLayer::BroadcastModified(EAnimNextEditorDataNotifType InType, bool bNotifyLayerLayoutChanges) const
{
	if (UUAFLayerStack_EditorData* OuterEditorData = Cast<UUAFLayerStack_EditorData>(GetOuter()))
	{
		OuterEditorData->BroadcastLayerStackChanged(InType, bNotifyLayerLayoutChanges);
	}
}

#if WITH_EDITOR
void UUAFLayer::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);
	
	if (PropertyAboutToChange && PropertyAboutToChange->GetName() == GET_MEMBER_NAME_CHECKED(UUAFLayer, LayerName))
	{
		PreRenameLayerName = LayerName;
	}
}

void UUAFLayer::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		bool bUpdateLayerLayout = false;
		if (PropertyChangedEvent.MemberProperty)
		{
			if (PropertyChangedEvent.MemberProperty->GetName() == GET_MEMBER_NAME_CHECKED(UUAFLayer, LayerName))
			{
				if (!IsLayerNameValid(LayerName) && !PreRenameLayerName.IsNone())
				{
					UE_LOGFMT(LogAnimation, Error, "Layer Name {LayerName} is invalid, will reset to previous name {PreLayerName}", LayerName, PreRenameLayerName);
					LayerName = PreRenameLayerName;
				}
			}
		}
	
		if (PropertyChangedEvent.PropertyChain.GetHead())
		{
			if (const FProperty* const Property = PropertyChangedEvent.PropertyChain.GetHead()->GetValue())
			{
				if (Property->GetName() == GET_MEMBER_NAME_CHECKED(UUAFLayer, LayerBlendProvider) 
				|| Property->GetName() == GET_MEMBER_NAME_CHECKED(UUAFLayer, LayerContentProvider))
				{
					bUpdateLayerLayout = true;
				}
			}
		}	
	
		BroadcastModified(EAnimNextEditorDataNotifType::PropertyChanged, bUpdateLayerLayout);
	}
}

void UUAFLayer::GetObjectReferences(TArray<const UObject*>& OutReferencedObjects) const
{
	if (IsLayerBlendProviderValid())
	{
		LayerBlendProvider.GetPtr()->GetObjectReferences(OutReferencedObjects);
	}

	if (IsLayerContentProviderValid())
	{
		LayerContentProvider.GetPtr()->GetObjectReferences(OutReferencedObjects);
	}
}
#endif

void UUAFLayer::RenameLayer(const FName NewName)
{
	LayerName = NewName;
}

bool UUAFLayer::IsLayerNameValid(const FName NewName) const
{
	if (const UUAFLayerStack_EditorData* EditorData = Cast<UUAFLayerStack_EditorData>(GetOuter()))
	{
		return EditorData->IsLayerNameValid(this, NewName);
	}
	
	return false;
}

EUAFLayerState UUAFLayer::GetLayerState() const
{
	return LayerState;
}

void UUAFLayer::SetLayerState(const EUAFLayerState InLayerState)
{
	this->LayerState = InLayerState;
}

bool UUAFLayer::IsLayerContentProviderValid() const
{
	return LayerContentProvider.IsValid();
}

FName UUAFLayer::GetLayerName() const
{
	return LayerName;
}

TInstancedStruct<FUAFLayerBlendProviderBase>& UUAFLayer::GetLayerBlendProvider()
{
	return LayerBlendProvider;
}

void UUAFLayer::SetLayerBlendProvider(const TInstancedStruct<FUAFLayerBlendProviderBase>& InLayerBlendProvider)
{
	LayerBlendProvider = InLayerBlendProvider;
}

bool UUAFLayer::IsLayerBlendProviderValid() const
{
	return LayerBlendProvider.IsValid();
}

TInstancedStruct<FUAFLayerContentProviderBase>& UUAFLayer::GetLayerContentProvider()
{
	return LayerContentProvider;
}

void UUAFLayer::SetLayerContentProvider(const TInstancedStruct<FUAFLayerContentProviderBase>& InLayerContentProvider)
{
	LayerContentProvider = InLayerContentProvider;
}


TSharedRef<SWidget> UUAFLayer::CreateLayerWidget()
{
	return SNew(UE::UAF::Layering::SLayer)
		.Layer(this);
	
}

#undef LOCTEXT_NAMESPACE
