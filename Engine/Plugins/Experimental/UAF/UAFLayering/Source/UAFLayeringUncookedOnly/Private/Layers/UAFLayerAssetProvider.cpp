// Copyright Epic Games, Inc. All Rights Reserved.


#include "Layers/UAFLayerAssetProvider.h"

#include "AnimNextAnimGraphSettings.h"
#include "AnimNextController.h"
#include "AnimNextRigVMAssetEditorData.h"
#include "AssetThumbnail.h"
#include "PropertyCustomizationHelpers.h"
#include "ScopedTransaction.h"
#include "Factory/AnimGraphFactory.h"
#include "Graph/RigUnit_AnimNextTraitStack.h"
#include "Layers/UAFLayer.h"
#include "RigVMCore/RigVMStruct.h"
#include "RigVMModel/Nodes/RigVMUnitNode.h"
#include "TraitCore/Trait.h"
#include "Traits/BlendStackTrait.h"
#include "Traits/LayerAssetDataTraitData.h"
#include "UAF/UAFAssetFactory.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "UAFLayerAssetContentProvider"

URigVMPin* FUAFLayerAssetProvider::CreateLayerContentTrait(UE::UAF::Layering::FLayerCreationContext& LayerCreationContext)
{
	LayerCreationContext.LastNodeLocation += FVector2D(0.0f, 500.0f);
	URigVMUnitNode* ContentHostTraitStackNode = LayerCreationContext.GraphController->AddUnitNode(FRigUnit_AnimNextTraitStack::StaticStruct(), FRigVMStruct::ExecuteName, LayerCreationContext.LastNodeLocation, FString(), false);
	if (ContentHostTraitStackNode == nullptr)
	{
		LayerCreationContext.CompileSettings.ReportError(TEXT("Failed to create trait stack to host layer content."));
		return nullptr;
	}
	
	// Create blend stack trait
	const FName BlendStackTraitName = LayerCreationContext.GraphController->AddTraitStruct(ContentHostTraitStackNode, TInstancedStruct<FAnimNextBlendStackCoreTraitSharedData>::Make(), INDEX_NONE, false, false);
	if (BlendStackTraitName.IsNone())
	{
		LayerCreationContext.CompileSettings.ReportError(TEXT("Failed to create Blend Stack Trait."));
		return nullptr;
	}
	
	// Create asset data provider trait 
	FUAFLayerAssetDataTraitSharedData AssetDataDefaults;
	AssetDataDefaults.GraphAssetHandle = AssetData;
	
	const FName AssetDataTraitName = LayerCreationContext.GraphController->AddTraitStruct(ContentHostTraitStackNode, TInstancedStruct<FUAFLayerAssetDataTraitSharedData>::Make(AssetDataDefaults), INDEX_NONE, false, false);
	if (AssetDataTraitName.IsNone())
	{
		LayerCreationContext.CompileSettings.ReportError(TEXT("Failed to create LayerAssetData Trait."));
		return nullptr;
	}
	
	// Expand trait pin per default
	URigVMPin* TraitPin = ContentHostTraitStackNode->FindTrait(AssetDataTraitName);
	LayerCreationContext.GraphController->SetPinExpansion(TraitPin->GetPinPath(), true, false);
	
	// Return the result pin of the trait stack to allow building of the graph
	URigVMPin* BlendTraitStackResult = ContentHostTraitStackNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_AnimNextTraitStack, Result));
	if (BlendTraitStackResult == nullptr)
	{
		LayerCreationContext.CompileSettings.ReportError(TEXT("Failed to retrieve Result pin from trait stack"));
		return nullptr;
	}
	
	return BlendTraitStackResult;
}

TSharedRef<SWidget> FUAFLayerAssetProvider::CreateLayerContentWidget(UUAFLayer* InLayer)
{
	OuterLayer = InLayer;

	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(EVerticalAlignment::VAlign_Center)
		[
			SNew(SObjectPropertyEntryBox)
				.AllowedClass(UObject::StaticClass())
				.DisplayThumbnail(true)
				.DisplayCompactSize(false)
				.OnShouldFilterAsset_Lambda([](const FAssetData& AssetData)
					{
						TArray<UClass*> AllowedClasses = UAnimNextAnimGraphSettings::GetAllowedAssetClasses();
						return (AllowedClasses.Contains(AssetData.GetClass()) == false);
					})
				.ObjectPath_Lambda([this]()
					{
						if (AssetData.IsValid())
						{
							TArray<const UObject*> OutReferencedObjects;
							AssetData.GetPtr()->GetObjectReferences(OutReferencedObjects);
							
							return OutReferencedObjects.Num() > 0 ? OutReferencedObjects[0]->GetPathName() : FString();
						}
						return FString();
					})
				.OnObjectChanged_Raw(this, &FUAFLayerAssetProvider::OnLayerAssetSelectionChanged)
				.ThumbnailPool(MakeShared<FAssetThumbnailPool>(1))
				
		];
}

void FUAFLayerAssetProvider::SetLayerAsset(const UObject* InNewAsset)
{
	const FScopedTransaction Transaction(LOCTEXT("OnLayerAssetSelectionChangedAction", "Set Layer Asset"));
	if (OuterLayer.IsValid())
	{
		OuterLayer->Modify();
	}

	if (InNewAsset)
	{
		AssetData = UE::UAF::FAssetDataFactory::CreateUAFAssetDataFromObject<FUAFGraphFactoryAsset>(InNewAsset);
	}
	else
	{
		AssetData.Reset();
	}
	
	if (OuterLayer.IsValid())
	{
		OuterLayer->BroadcastModified(EAnimNextEditorDataNotifType::PropertyChanged);
	}
}

void FUAFLayerAssetProvider::OnLayerAssetSelectionChanged(const FAssetData& InAssetData)
{
	UObject* Obj = Cast<UObject>(InAssetData.GetAsset());
	SetLayerAsset(Obj);
}

#if WITH_EDITOR
void FUAFLayerAssetProvider::GetObjectReferences(TArray<const UObject*>& OutReferencedObjects) const
{
	if (AssetData.IsValid())
	{
		AssetData.GetPtr()->GetObjectReferences(OutReferencedObjects);
	}
}
#endif

#if WITH_EDITORONLY_DATA
void FUAFLayerAssetProvider::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (LayerAsset_DEPRECATED != nullptr && AssetData.IsValid() == false)
		{
			AssetData = UE::UAF::FAssetDataFactory::CreateUAFAssetDataFromObject<FUAFGraphFactoryAsset>(LayerAsset_DEPRECATED);
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}
#endif

#undef LOCTEXT_NAMESPACE //UAFLayerAssetContentProvider

