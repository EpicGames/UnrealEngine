// Copyright Epic Games, Inc. All Rights Reserved.


#include "Layers/UAFLayerDefaultBlendProvider.h"

#include "AnimNextController.h"
#include "AnimNextRigVMAssetEditorData.h"
#include "ISinglePropertyView.h"
#include "UAFLayeringStyle.h"
#include "UAFLayerStack.h"
#include "UAF/BlendMask/UAFBlendMask.h"
#include "UAF/BlendProfile/UAFBlendProfile.h"
#include "Graph/RigUnit_AnimNextTraitStack.h"
#include "Layers/UAFLayer.h"
#include "RigVMCore/RigVMStruct.h"
#include "RigVMModel/Nodes/RigVMUnitNode.h"
#include "TraitCore/Trait.h"
#include "Traits/ApplyAdditivePerBoneTraitData.h"
#include "Traits/ApplyAdditiveTraitData.h"
#include "Traits/ApplyNamedSetTraitData.h"
#include "Traits/BlendLayerTraitData.h"
#include "Traits/BlendTwoWayTraitData.h"
#include "Traits/CachePoseTraitData.h"
#include "Traits/LayerDataProviderTraitData.h"
#include "Traits/PassthroughTraitData.h"
#include "Widgets/SDefaultBlendProvider.h"

#define LOCTEXT_NAMESPACE "UAFDefaultBlendProvider"


URigVMPin* FUAFDefaultBlendProvider::CreateBlendGraphTrait(UE::UAF::Layering::FLayerCreationContext& LayerCreationContext)
{
	// In some cases we may have to create 2 trait stacks to allow for some of the more advanced features - mostly because currently we cannot inject tasks between child trait evals/updates 
	// Case 1: Adding a named set to only evaluate what a layer needs, this needs to happen in between child trait evals (applies not to base but needs to apply to layer input) 
	// Case 2: Cache Only Layers - We need to be able to pop the layer content keyframe and cache it at the right point to ensure the keyframe stack is in an expected state 
	
	// Create pre blend trait stack if needed (e.g. Cache Only or to provide name set optimization) 
	if (BlendMask || BlendMode == EUAFLayerBlendMode::CacheOnly)
	{
		// Create Pre-Blend trait stack 
		LayerCreationContext.LastNodeLocation += FVector2D(500.0f, 150.0f);
		URigVMUnitNode* PreBlendTraitStack = LayerCreationContext.GraphController->AddUnitNode(FRigUnit_AnimNextTraitStack::StaticStruct(), FRigVMStruct::ExecuteName, LayerCreationContext.LastNodeLocation, FString(), false);
		if (PreBlendTraitStack == nullptr)
		{
			LayerCreationContext.CompileSettings.ReportError(TEXT("Failed to create pre-blend trait stack."));
			return nullptr;
		}
		
		// The input pin for the pre-blend trait stack. This will depend on which traits are created
		URigVMPin* PreBlendInput = nullptr;

		if (BlendMask)
		{
			// Create Apply Named Set Trait - this is a base trait, so no need to create an additional pass through trait 
			UE::UAF::FApplyNamedSetSharedData DefaultData;
			DefaultData.SetName = SetName;
			const FName ApplyNamedSetTraitName = LayerCreationContext.GraphController->AddTraitStruct(PreBlendTraitStack, TInstancedStruct<UE::UAF::FApplyNamedSetSharedData>::Make(DefaultData), INDEX_NONE, false, false);
			if (ApplyNamedSetTraitName == NAME_None)
			{
				LayerCreationContext.CompileSettings.ReportError(TEXT("Failed to create Apply Named Set trait."));
				return nullptr;
			}
			
			// Hook up the input pin to the input of the Apply Named Set trait 
			PreBlendInput = PreBlendTraitStack->FindTrait(ApplyNamedSetTraitName, GET_MEMBER_NAME_STRING_CHECKED(UE::UAF::FApplyNamedSetSharedData, Input));
		}
		
		if (BlendMode == EUAFLayerBlendMode::CacheOnly)
		{
			// If we haven't created a trait yet to take care of the input we'll add a passthrough onto the trait stack 
			if (PreBlendInput == nullptr)
			{
				// Create passthrough trait 
				const FName PreBlendPassThroughTraitName = LayerCreationContext.GraphController->AddTraitStruct(PreBlendTraitStack, TInstancedStruct<FAnimNextPassthroughSharedData>::Make(), INDEX_NONE, false, false);
		
				// Populate the new input pin
				PreBlendInput = PreBlendTraitStack->FindTrait(PreBlendPassThroughTraitName, GET_MEMBER_NAME_STRING_CHECKED(FAnimNextPassthroughSharedData, Input));
			}
			
			// Create cache pose additive trait 
			const FName CachePoseTraitName = LayerCreationContext.GraphController->AddTraitStruct(PreBlendTraitStack, TInstancedStruct<FUAFCachePoseTraitSharedData>::Make(), INDEX_NONE, false, false);
			if (CachePoseTraitName == NAME_None)
			{
				LayerCreationContext.CompileSettings.ReportError(TEXT("Failed to create Cache Pose trait."));
				return nullptr;
			}
		}

		if (PreBlendInput && PreBlendTraitStack)
		{
			// Link current layer asset with passthrough
			const bool bLayerContentLinkCreated = LayerCreationContext.GraphController->AddLink(LayerCreationContext.LayerInputs[1], PreBlendInput, false);
			if (!bLayerContentLinkCreated)
			{
				LayerCreationContext.CompileSettings.ReportError(TEXT("Failed to link between layer content output and pre-blend trait"));
				return nullptr;
			}
			
			// Update input so the layer trait stack can link correctly 
			URigVMPin* PreBlendTraitStackResult = PreBlendTraitStack->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_AnimNextTraitStack, Result));
			LayerCreationContext.LayerInputs[1] = PreBlendTraitStackResult;
		}
	}
	
	URigVMPin* ChildAInputPin = nullptr;
	URigVMPin* ChildBInputPin = nullptr;
	
	// Create Blend Trait Stack Node
	LayerCreationContext.LastNodeLocation += FVector2D(500.0f, -150.0f);
	URigVMUnitNode* BlendTraitStackNode = LayerCreationContext.GraphController->AddUnitNode(FRigUnit_AnimNextTraitStack::StaticStruct(), FRigVMStruct::ExecuteName, LayerCreationContext.LastNodeLocation, FString(), false);
	if (BlendTraitStackNode == nullptr)
	{
		LayerCreationContext.CompileSettings.ReportError(TEXT("Failed to create trait stack for the layer default blend implementation."));
		return nullptr;
	}
	
	FName BlendBaseTraitName = NAME_None;
	const bool bUsesPerBoneData = BlendMask || BlendProfile;
	if (BlendMode == EUAFLayerBlendMode::Additive)
	{
		if (bUsesPerBoneData)
		{
			UE::UAF::FApplyAdditivePerBoneTraitSharedData DefaultAdditiveTraitData;
			DefaultAdditiveTraitData.Alpha = LayerWeight;
			DefaultAdditiveTraitData.BlendMask = BlendMask;
			
			// Create trait 
			BlendBaseTraitName = LayerCreationContext.GraphController->AddTraitStruct(BlendTraitStackNode, TInstancedStruct<UE::UAF::FApplyAdditivePerBoneTraitSharedData>::Make(DefaultAdditiveTraitData), INDEX_NONE, false, false);

			// Set Input Pins so links to this trait stack can be generated 
			ChildAInputPin = BlendTraitStackNode->FindTrait(BlendBaseTraitName, GET_MEMBER_NAME_STRING_CHECKED(UE::UAF::FApplyAdditivePerBoneTraitSharedData, Base));
			ChildBInputPin = BlendTraitStackNode->FindTrait(BlendBaseTraitName, GET_MEMBER_NAME_STRING_CHECKED(UE::UAF::FApplyAdditivePerBoneTraitSharedData, Additive));
		}
		else
		{
			FAnimNextApplyAdditiveTraitSharedData DefaultAdditiveTraitData;
			DefaultAdditiveTraitData.Alpha = LayerWeight;
		
			// Create trait 
			BlendBaseTraitName = LayerCreationContext.GraphController->AddTraitStruct(BlendTraitStackNode, TInstancedStruct<FAnimNextApplyAdditiveTraitSharedData>::Make(DefaultAdditiveTraitData), INDEX_NONE, false, false);

			// Set Input Pins so links to this trait stack can be generated 
			ChildAInputPin = BlendTraitStackNode->FindTrait(BlendBaseTraitName, GET_MEMBER_NAME_STRING_CHECKED(FAnimNextApplyAdditiveTraitSharedData, Base));
			ChildBInputPin = BlendTraitStackNode->FindTrait(BlendBaseTraitName, GET_MEMBER_NAME_STRING_CHECKED(FAnimNextApplyAdditiveTraitSharedData, Additive));
		}
	}
	else if (BlendMode == EUAFLayerBlendMode::Blend)
	{
		if (bUsesPerBoneData)
		{
			// Setup default trait values 
			FUAFBlendLayerTraitSharedData DefaultTraitData; 
			DefaultTraitData.BlendWeight = LayerWeight;
			DefaultTraitData.BlendMask = BlendMask;
			DefaultTraitData.bAlwaysUpdateChildBlend = GetAlwaysUpdateChildren();
			
			// Create trait 
			BlendBaseTraitName = LayerCreationContext.GraphController->AddTraitStruct(BlendTraitStackNode, TInstancedStruct<FUAFBlendLayerTraitSharedData>::Make(DefaultTraitData), INDEX_NONE, false, false);

			// Set Input Pins so links to this trait stack can be generated 
			ChildAInputPin = BlendTraitStackNode->FindTrait(BlendBaseTraitName, GET_MEMBER_NAME_STRING_CHECKED(FUAFBlendLayerTraitSharedData, ChildBase));
			ChildBInputPin = BlendTraitStackNode->FindTrait(BlendBaseTraitName, GET_MEMBER_NAME_STRING_CHECKED(FUAFBlendLayerTraitSharedData, ChildBlend));
		}
		else
		{
			// Setup default trait values 
			FAnimNextBlendTwoWayTraitSharedData DefaultTraitData; 
			DefaultTraitData.BlendWeight = LayerWeight;
			DefaultTraitData.bAlwaysUpdateChildren = GetAlwaysUpdateChildren();
			
			// Create two-way blend trait 
			BlendBaseTraitName = LayerCreationContext.GraphController->AddTraitStruct(BlendTraitStackNode, TInstancedStruct<FAnimNextBlendTwoWayTraitSharedData>::Make(DefaultTraitData), INDEX_NONE, false, false);

			// Set Input Pins so links to this trait stack can be generated 
			ChildAInputPin = BlendTraitStackNode->FindTrait(BlendBaseTraitName, GET_MEMBER_NAME_STRING_CHECKED(FAnimNextBlendTwoWayTraitSharedData, ChildA));
			ChildBInputPin = BlendTraitStackNode->FindTrait(BlendBaseTraitName, GET_MEMBER_NAME_STRING_CHECKED(FAnimNextBlendTwoWayTraitSharedData, ChildB));
		}
	}
	else if (BlendMode == EUAFLayerBlendMode::CacheOnly)
	{
		// Create passthrough trait on the blend trait stack - since we won't be performing a blend when Cache Only is set  
		BlendBaseTraitName = LayerCreationContext.GraphController->AddTraitStruct(BlendTraitStackNode, TInstancedStruct<FAnimNextPassthroughSharedData>::Make(), INDEX_NONE, false, false);
		
		ChildAInputPin = BlendTraitStackNode->FindTrait(BlendBaseTraitName, GET_MEMBER_NAME_STRING_CHECKED(FAnimNextPassthroughSharedData, Input));
	}

	if (BlendBaseTraitName == NAME_None)
	{
		LayerCreationContext.CompileSettings.ReportError(TEXT("Failed to create the base trait for the blend trait stack!"));
		return nullptr;
	}
	
	// Expand trait pin per default 
	if (const URigVMPin* TraitPin = BlendTraitStackNode->FindTrait(BlendBaseTraitName))
	{
		LayerCreationContext.GraphController->SetPinExpansion(TraitPin->GetPinPath(), true, false);
	}
	
	// Create the layer data provider 
	{
		// Build runtime data on trait 
		UUAFLayerStack_EditorData* EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData<UUAFLayerStack_EditorData>(LayerCreationContext.LayerStack.Get());
		const int32 LayerIndex = EditorData->GetIndexForLayer(LayerCreationContext.Layer);
		
		// TODO: This data, which contains the default/bound properties of a layer will likely not live on the trait itself like this in the future once bindings are implemented
		FUAFLayerProperties LayerProperties; 
		LayerProperties.bLayerEnabled = bLayerEnabled;
		LayerProperties.DesiredLayerWeight = LayerWeight;
		LayerProperties.BlendInTime = LayerBlendInTime;
		LayerProperties.BlendOutTime = LayerBlendOutTime;
		LayerProperties.BlendCurve = BlendCurve;
		LayerProperties.BlendOption = BlendOption;
		
		// Setup default trait values 
		FUAFLayerDataProviderTraitSharedData DefaultTraitData; 
		DefaultTraitData.LayerName = LayerCreationContext.Layer->GetLayerName();
		DefaultTraitData.LayerIndex = LayerIndex;
		DefaultTraitData.bCreateCacheInput = BlendMode == EUAFLayerBlendMode::CacheOnly;
		DefaultTraitData.LayerStackPath = FSoftObjectPath::ConstructFromObject(LayerCreationContext.LayerStack.Get()).ToString();
		DefaultTraitData.LayerProperties = MoveTemp(LayerProperties);
		
		const FName LayerDataProviderTraitName = LayerCreationContext.GraphController->AddTraitStruct(BlendTraitStackNode, TInstancedStruct<FUAFLayerDataProviderTraitSharedData>::Make(DefaultTraitData), INDEX_NONE, false, false);
		if (LayerDataProviderTraitName == NAME_None)
		{
			LayerCreationContext.CompileSettings.ReportError(TEXT("Could not create the LayerDataProviderTrait."));
			return nullptr;
		}
		
		// Expand trait pin per default 
		const URigVMPin* TraitPin = BlendTraitStackNode->FindTrait(LayerDataProviderTraitName);
		LayerCreationContext.GraphController->SetPinExpansion(TraitPin->GetPinPath(), true, false);
		
		// If we are in cache only mode, hook up the layer content output pin to that handle 
		if (BlendMode == EUAFLayerBlendMode::CacheOnly)
		{
			ChildBInputPin = BlendTraitStackNode->FindTrait(LayerDataProviderTraitName, GET_MEMBER_NAME_STRING_CHECKED(FUAFLayerDataProviderTraitSharedData, CacheOnlyInput));
		}
	}
	
	// Link input pins to blend trait 
	// Previous Layer Output --> Child A 
	const bool bLayerBaseInputLinkCreated = LayerCreationContext.GraphController->AddLink(LayerCreationContext.LayerInputs[0], ChildAInputPin, false);
	// Layer Output --> Child B
	const bool bLayerContentLinkCreated = LayerCreationContext.GraphController->AddLink(LayerCreationContext.LayerInputs[1], ChildBInputPin, false);

	if (!bLayerBaseInputLinkCreated || !bLayerContentLinkCreated)
	{
		LayerCreationContext.CompileSettings.ReportError(TEXT("Failed to create links to blend trait!"));
		return nullptr;
	}
	
	URigVMPin* BlendTraitStackResult = BlendTraitStackNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_AnimNextTraitStack, Result));
	if (BlendTraitStackResult == nullptr)
	{
		LayerCreationContext.CompileSettings.ReportError(TEXT("Failed to retrieve Result pin from trait stack"));
		return nullptr;
	}
	
	return BlendTraitStackResult;
}


TSharedRef<SWidget> FUAFDefaultBlendProvider::CreateLayerBlendWidget(UUAFLayer* InLayer)
{
	return SNew(UE::UAF::Layering::SDefaultBlendProvider)
		.Layer(InLayer);
}

const FSlateBrush* FUAFDefaultBlendProvider::GetOverrideLayerBackground() const
{
	switch (BlendMode)
	{
	case EUAFLayerBlendMode::CacheOnly:
		return FUAFLayeringStyle::Get().GetBrush(TEXT("Layer.CacheOnlyBackground"));
	default:
		return nullptr;
	}
}

bool FUAFDefaultBlendProvider::GetOverrideIndicatorColor(FSlateColor& OutSlateColor) const
{
	switch (BlendMode)
	{
	case EUAFLayerBlendMode::Blend:
		OutSlateColor = FUAFLayeringStyle::Get().GetSlateColor("Layer.Colors.BlendIndictor");
		return true;
	case EUAFLayerBlendMode::Additive:
		OutSlateColor = FUAFLayeringStyle::Get().GetSlateColor("Layer.Colors.AdditiveIndictor");
		return true;
	case EUAFLayerBlendMode::CacheOnly:
		OutSlateColor = FUAFLayeringStyle::Get().GetSlateColor("Layer.Colors.CacheOnlyIndictor");
		return true;
	case EUAFLayerBlendMode::MAX:
	default:
		return false;
	}
}

namespace UE::UAF::Layering
{
bool FDefaultLayerBlendStructureDataProvider::IsValid() const
{
	return BlendProvider != nullptr && BlendProvider->IsValid();
}

const UStruct* FDefaultLayerBlendStructureDataProvider::GetBaseStructure() const
{
	return FUAFDefaultBlendProvider::StaticStruct();
}

void FDefaultLayerBlendStructureDataProvider::GetInstances(TArray<TSharedPtr<FStructOnScope>>& OutInstances, const UStruct* ExpectedBaseStructure) const
{
	if (IsValid())
	{
		const bool bCompatibleStructType = BlendProvider->GetScriptStruct() == ExpectedBaseStructure;
		if (bCompatibleStructType)
		{
			uint8* Memory = BlendProvider->GetMutableMemory();
			OutInstances.Add(MakeShareable(new FStructOnScope(BlendProvider->GetScriptStruct(), Memory)));
		}
	}
}
}

#undef LOCTEXT_NAMESPACE //UAFDefaultBlendProvider
