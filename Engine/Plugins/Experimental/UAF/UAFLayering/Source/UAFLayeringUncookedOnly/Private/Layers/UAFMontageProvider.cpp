// Copyright Epic Games, Inc. All Rights Reserved.


#include "Layers/UAFMontageProvider.h"

#include "AnimNextController.h"
#include "IDetailTreeNode.h"
#include "UAFLayerStack.h"
#include "UAFLayerStack_EditorData.h"
#include "Components/HorizontalBox.h"
#include "Graph/RigUnit_AnimNextTraitStack.h"
#include "Injection/MontageTrait.h"
#include "Layers/UAFLayer.h"
#include "Traits/MontageLayerTraitData.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "UAFMontageProvider"

URigVMPin* FUAFMontageProvider::CreateLayerContentTrait(UE::UAF::Layering::FLayerCreationContext& LayerCreationContext)
{
	LayerCreationContext.LastNodeLocation += FVector2D(0.0f, 500.0f);
	
	// Create trait stack
	URigVMUnitNode* MontageTraitStackNode = LayerCreationContext.GraphController->AddUnitNode(FRigUnit_AnimNextTraitStack::StaticStruct(), FRigVMStruct::ExecuteName, LayerCreationContext.LastNodeLocation, FString(), false);
	if (MontageTraitStackNode == nullptr)
	{
		LayerCreationContext.CompileSettings.ReportError(TEXT("Failed to create trait stack to host layer content."));
		return nullptr;
	}
	
	// Create legacy montage trait 
	FUAFMontageTraitSharedData DataDefaults;
	DataDefaults.SlotName = SlotName;
	DataDefaults.bAlwaysUpdateSource = bAlwaysUpdateSource;
	DataDefaults.bDisableSynchronization = bDisableSynchronization;
	
	const FName MontageTraitName = LayerCreationContext.GraphController->AddTraitStruct(MontageTraitStackNode, TInstancedStruct<FUAFMontageTraitSharedData>::Make(DataDefaults), INDEX_NONE, false, false);
	if (MontageTraitName.IsNone())
	{
		LayerCreationContext.CompileSettings.ReportError(TEXT("Failed to create Montage Trait."));
		return nullptr;
	}
	
	// Create additive montage trait if required
	if (bAutoEnableLayerWithMontage || bAutoSetBlendTimesFromMontage)
	{
		FUAFMontageLayerTraitSharedData LayerDataDefaults;
		LayerDataDefaults.bAutoEnableLayer = bAutoEnableLayerWithMontage;
		LayerDataDefaults.bAutoSetBlendTimes = bAutoSetBlendTimesFromMontage;
		LayerDataDefaults.SlotName = SlotName;
		LayerDataDefaults.LayerName = LayerCreationContext.Layer->GetLayerName();
		LayerDataDefaults.LayerStackPath = FSoftObjectPath::ConstructFromObject(LayerCreationContext.LayerStack.Get()).ToString();
	
		const FName MontageLayerTraitName = LayerCreationContext.GraphController->AddTraitStruct(MontageTraitStackNode, TInstancedStruct<FUAFMontageLayerTraitSharedData>::Make(LayerDataDefaults), INDEX_NONE, false, false);
		if (MontageLayerTraitName.IsNone())
		{
			LayerCreationContext.CompileSettings.ReportError(TEXT("Failed to create Montage Layer Trait."));
			return nullptr;
		}

		if (bAutoEnableLayerWithMontage && LayerCreationContext.Layer->IsLayerBlendProviderValid())
		{
			// if we want to be able to modify the layers behavior even if it is currently not enabled (e.g. to auto enable it) we have to always update it
			LayerCreationContext.Layer->GetLayerBlendProvider().GetMutablePtr()->SetAlwaysUpdateChildren(true);
		}
	}
	
	// return result pin to connect to the rest of the graph 
	URigVMPin* BlendTraitStackResult = MontageTraitStackNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_AnimNextTraitStack, Result));
	if (BlendTraitStackResult == nullptr)
	{
		LayerCreationContext.CompileSettings.ReportError(TEXT("Failed to retrieve Result pin from trait stack"));
		return nullptr;
	}
	
	return BlendTraitStackResult;
}

TSharedRef<SWidget> FUAFMontageProvider::CreateLayerContentWidget(UUAFLayer* InLayer)
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(EVerticalAlignment::VAlign_Center)
		[
			SNew(SEditableTextBox)
			.ToolTipText(LOCTEXT("SlotNameLabel", "The name of the montage slot"))
			.Text(FText::FromName(SlotName))
			.OnTextCommitted_Lambda([this](const FText& Val, ETextCommit::Type TextCommitType)
			{
				SlotName = FName(Val.ToString());
			})
		];
}

#undef LOCTEXT_NAMESPACE //UAFMontageProvider
