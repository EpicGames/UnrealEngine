// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAFLayerStack_EditorData.h"
#include "AnimGraphUncookedOnlyUtils.h"
#include "Editor.h"
#include "ScopedTransaction.h"
#include "UAFLayerStack.h"
#include "Compilation/AnimNextGetGraphCompileContext.h"
#include "Entries/AnimNextAnimationGraphEntry.h"
#include "Entries/AnimNextSharedVariablesEntry.h"
#include "Entries/AnimNextVariableEntry.h"
#include "Graph/AnimNextAnimationGraphSchema.h"
#include "Graph/RigUnit_AnimNextGraphRoot.h"
#include "Layers/UAFBaseLayer.h"
#include "Layers/UAFLayer.h"
#include "Layers/UAFLayerAssetProvider.h"
#include "Layers/UAFLayerDefaultBlendProvider.h"
#include "Workspace/UAFLayerStackWorkspaceAssetUserData.h"

#define LOCTEXT_NAMESPACE "UUAFLayerStack_EditorData"

UUAFLayerStack_EditorData::UUAFLayerStack_EditorData()
{
	if (GEditor)
	{
		GEditor->RegisterForUndo(this);
	}
}

UUAFLayerStack_EditorData::~UUAFLayerStack_EditorData()
{
	if (GEditor)
	{
		GEditor->UnregisterForUndo(this);
	}
}

void UUAFLayerStack_EditorData::PostUndo(bool bSuccess)
{
	if (bSuccess)
	{
		BroadcastLayerStackChanged(EAnimNextEditorDataNotifType::UndoRedo, true);
	}
}

void UUAFLayerStack_EditorData::PostRedo(bool bSuccess)
{
	PostUndo(bSuccess);
}

void UUAFLayerStack_EditorData::OnPreCompileGetProgrammaticGraphs(const FRigVMCompileSettings& InSettings, FAnimNextGetGraphCompileContext& OutCompileContext)
{
	Super::OnPreCompileGetProgrammaticGraphs(InSettings, OutCompileContext);
	
	if (UUAFLayerStack* LayerStack = UE::UAF::UncookedOnly::FUtils::GetAsset<UUAFLayerStack>(this))
	{
		// Create procedural RigVM Graph to host our layer content and blend logic 
		URigVMGraph* Graph = NewObject<URigVMGraph>(this, NAME_None, RF_Transient);
		Graph->SetSchemaClass(UAnimNextAnimationGraphSchema::StaticClass());

		// Create Controller to manipulate the created graph
		UAnimNextController* Controller = CastChecked<UAnimNextController>(RigVMClient.GetOrCreateController(Graph));

		// Setup basic graph so we can add our custom trait stacks
		UE::UAF::UncookedOnly::FAnimGraphUtils::SetupAnimGraph(FRigUnit_AnimNextGraphRoot::DefaultEntryPoint, Controller, false);

		if (Controller->GetGraph()->GetNodes().Num() != 1)
		{
			InSettings.ReportError(TEXT("Failed to setup UAFAnimGraph - Expected singular FRigUnit_AnimNextGraphRoot node"));
			return;
		}

		URigVMNode* EntryNode = Controller->GetGraph()->GetNodes()[0];
		URigVMPin* GraphResultPin = EntryNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_AnimNextGraphRoot, Result));
		
		UE::UAF::Layering::FLayerCreationContext LayerCreationContext(InSettings);
		LayerCreationContext.LayerStack = LayerStack; 
		LayerCreationContext.LastNodeLocation = FVector2D::ZeroVector;
		LayerCreationContext.GraphController = Controller;

		URigVMPin* LayerOutputPin = nullptr;
		const TArray<TObjectPtr<UUAFLayer>> AllLayers = GetAllLayers();
		for (UUAFLayer* LayerToCreate : AllLayers)
		{
			if (!LayerToCreate || LayerToCreate->GetLayerState() == EUAFLayerState::Disabled)
			{
				continue;
			}
			
			LayerOutputPin = LayerToCreate->CreateLayerGraphContentTraits(LayerCreationContext);
			if (LayerOutputPin == nullptr)
			{
				InSettings.ReportErrorf(TEXT("Failed to create Layer %s"), *LayerToCreate->GetLayerName().ToString());
				return;
			}
			
			LayerCreationContext.LayerInputs[0] = LayerOutputPin;
		}

		// adjust location of results node 
		LayerCreationContext.LastNodeLocation += FVector2D(500.0f, 0.0f);
		Controller->SetNodePosition(EntryNode, LayerCreationContext.LastNodeLocation, false);

		// Link the final output to the graph result 
		const bool bLinkedResultNode = Controller->AddLink(LayerOutputPin, GraphResultPin, false);
		if (!bLinkedResultNode)
		{
			InSettings.ReportError(TEXT("Failed to link final Layer and Graph Output pin"));
			return;
		}

		CreatedGraph = Graph;
		OutCompileContext.GetMutableProgrammaticGraphs().Add(Graph);
	}
}

void UUAFLayerStack_EditorData::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	static const FString LayerPropertyString = GET_MEMBER_NAME_STRING_CHECKED(UUAFLayerStack_EditorData, Layers);
	if (PropertyChangedEvent.MemberProperty && PropertyChangedEvent.MemberProperty->GetName() == LayerPropertyString)
	{
		// Check if a new layer got added manually e.g. via the details panel 
		if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ValueSet)
		{
			if (PropertyChangedEvent.PropertyChain.GetTail())
			{
				if (const FProperty* const Property = PropertyChangedEvent.PropertyChain.GetTail()->GetValue())
				{
					if (Property->GetName() == LayerPropertyString)
					{
						const int32 ChangedIndex = PropertyChangedEvent.GetArrayIndex(LayerPropertyString);
						if (Layers.IsValidIndex(ChangedIndex))
						{
							OnLayerAdded(Layers[ChangedIndex]);
							return;
						}
					}
				}
			}
		}
		
		NotifyLayerLayoutChanged();
	}
}

void UUAFLayerStack_EditorData::OnLayerAdded(const TObjectPtr<UUAFLayer> NewLayer)
{
	if (NewLayer)
	{
		NewLayer->RenameLayer(GetUniqueNameForLayer(NewLayer));
		BroadcastLayerStackChanged(EAnimNextEditorDataNotifType::PropertyChanged, true);
	}
}

void UUAFLayerStack_EditorData::NotifyLayerLayoutChanged() const
{
	OnLayerLayoutChanged.Broadcast();
}

TArray<TObjectPtr<UUAFLayer>> UUAFLayerStack_EditorData::GetAllLayers() const
{
	TArray<TObjectPtr<UUAFLayer>> AllLayers;
	AllLayers.Add(BaseLayer);
	AllLayers.Append(Layers);
	
	return MoveTemp(AllLayers);
}

void UUAFLayerStack_EditorData::MoveLayerUp(const TObjectPtr<UUAFLayer> LayerToMove)
{
	const int32 SourceLayerIndex = Layers.Find(LayerToMove);
	if (SourceLayerIndex == INDEX_NONE)
	{
		// error, not a valid layer 
		return;
	}

	if (LayerToMove == BaseLayer)
	{
		// can not move the base layer 
		return;
	}

	if (SourceLayerIndex == 0)
	{
		// can not move this layer up, and it is already the uppermost layer after the base layer
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("MoveLayerUpAction", "Move Layer Up"));
	Modify();
	
	const int32 TargetIndex = SourceLayerIndex - 1;
	Layers.Swap(SourceLayerIndex, TargetIndex);

	BroadcastLayerStackChanged(EAnimNextEditorDataNotifType::PropertyChanged, true);
}

void UUAFLayerStack_EditorData::MoveLayerDown(const TObjectPtr<UUAFLayer> LayerToMove)
{
	const int32 SourceLayerIndex = Layers.Find(LayerToMove);
	if (SourceLayerIndex == INDEX_NONE)
	{
		// error, not a valid layer 
		return;
	}

	if (LayerToMove == BaseLayer)
	{
		// cant move base layer 
		return;
	}
	
	if (SourceLayerIndex == Layers.Num() -1)
	{
		// if layer is the last one it makes no sense to move it down. 
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("MoveLayerDownAction", "Move Layer Down"));
	Modify();
	
	const int32 TargetIndex = SourceLayerIndex + 1;
	Layers.Swap(SourceLayerIndex, TargetIndex);

	BroadcastLayerStackChanged(EAnimNextEditorDataNotifType::PropertyChanged, true);
}

void UUAFLayerStack_EditorData::RemoveLayer(const TObjectPtr<UUAFLayer> LayerToDelete)
{
	if (LayerToDelete)
	{
		if (LayerToDelete == BaseLayer)
		{
			// not allowed to remove the base layer
			return;
		}
		
		const FScopedTransaction Transaction(LOCTEXT("DeleteLayerAction", "Delete Layer"));
		Modify();
		
		Layers.Remove(LayerToDelete);
		
		BroadcastLayerStackChanged(EAnimNextEditorDataNotifType::PropertyChanged, true);
	}
}

void UUAFLayerStack_EditorData::SetLayerState(const TObjectPtr<UUAFLayer> Layer, EUAFLayerState LayerState)
{
	if (!Layer)
	{
		return;
	}
	
	const FScopedTransaction Transaction(LOCTEXT("SetLayerStateAction", "Set Layer State"));
	Modify();
	
	Layer->SetLayerState(LayerState);
	BroadcastModified(EAnimNextEditorDataNotifType::PropertyChanged, this);
}

void UUAFLayerStack_EditorData::SelectLayer(const TObjectPtr<UUAFLayer> Layer) const
{
	OnLayerSelectionChanged.Broadcast(Layer);
}

void UUAFLayerStack_EditorData::ClearSelectedLayer() const
{
	OnLayerSelectionChanged.Broadcast(nullptr);
}

int32 UUAFLayerStack_EditorData::GetIndexForLayer(const TObjectPtr<UUAFLayer> InLayer, const UE::UAF::Layering::EBaseLayerInclusion IncludeBaseLayer) const 
{
	switch (IncludeBaseLayer)
	{
	case UE::UAF::Layering::EBaseLayerInclusion::Include:
	{
		if (InLayer == BaseLayer)
		{
			return 0;
		}

		const int32 LayerIndex = Layers.Find(InLayer);
		return LayerIndex == INDEX_NONE ? INDEX_NONE : LayerIndex + 1; 
	}
	case UE::UAF::Layering::EBaseLayerInclusion::Exclude:
		return Layers.Find(InLayer);
	default:
		// Unhandled case
		return INDEX_NONE;
	}
}


int32 UUAFLayerStack_EditorData::GetNumLayers(const UE::UAF::Layering::EBaseLayerInclusion IncludeBaseLayer) const
{
	switch (IncludeBaseLayer)
	{
	case UE::UAF::Layering::EBaseLayerInclusion::Include:
		return GetAllLayers().Num();
	case UE::UAF::Layering::EBaseLayerInclusion::Exclude:
		return Layers.Num();
	default:
		// Unhandled case
		return INDEX_NONE;
	}
}

bool UUAFLayerStack_EditorData::IsBaseLayer(const TObjectPtr<UUAFLayer> InLayer) const
{
	const int32 LayerIndex = GetIndexForLayer(InLayer);
	return LayerIndex == 0;
}

bool UUAFLayerStack_EditorData::IsLastLayer(const TObjectPtr<UUAFLayer> InLayer) const
{
	const int32 LayerIndex = GetIndexForLayer(InLayer);
	return LayerIndex == GetAllLayers().Num() - 1;
}

bool UUAFLayerStack_EditorData::IsLayerNameValid(const TObjectPtr<const UUAFLayer> InLayer, const FName NewLayerName) const
{
	// Empty layer names are not allowed 
	if (NewLayerName == NAME_None)
	{
		return false;
	}
	
	// If renaming a specific layer, it is allowed to keep the current name 
	if (InLayer && InLayer->GetLayerName() == NewLayerName)
	{
		return true;
	}
	
	// Layer names need to be unique within the layer stack 
	for (const UUAFLayer* Layer : GetAllLayers())
	{
		if (!Layer || Layer == InLayer)
		{
			continue;
		}
		
		if (Layer->GetLayerName() == NewLayerName)
		{
			return false;
		}
	}
	return true;
}

FName UUAFLayerStack_EditorData::GetUniqueNameForLayer(const TObjectPtr<UUAFLayer> InLayer) const
{
	if (!InLayer)
	{
		return NAME_None;
	}
	
	// Check if the current name is already unique 
	if (IsLayerNameValid(InLayer, InLayer->GetLayerName()))
	{
		return InLayer->GetLayerName();
	}
	
	// Otherwise try and name the layer based on its index in the layer stack
	int32 LayerIndex = GetIndexForLayer(InLayer);
	if (LayerIndex == INDEX_NONE)
	{
		// If the given layer isn't in the layer stack yet, use the next index
		LayerIndex = GetAllLayers().Num();
	}
	
	static const FText LayerPrefix = LOCTEXT("LayerNamePrefix", "Layer");
	FName NewLayerName = FName(FString::Printf(TEXT("%s%d"), *LayerPrefix.ToString(), LayerIndex));
	int32 IterationIndex = 0;
	while (!IsLayerNameValid(InLayer, NewLayerName))
	{
		NewLayerName = FName(FString::Printf(TEXT("%s%d_%d"), *LayerPrefix.ToString(), LayerIndex, IterationIndex));
		++IterationIndex;
	}
	
	return NewLayerName;
}

void UUAFLayerStack_EditorData::MoveLayerToIndex(const TObjectPtr<UUAFLayer> LayerToMove, int32 NewLayerIndex)
{
	if (LayerToMove && !Layers.IsEmpty())
	{
		NewLayerIndex = FMath::Clamp(NewLayerIndex, 0, Layers.Num() - 1);
		int32 OriginalLayerIndex = GetIndexForLayer(LayerToMove, UE::UAF::Layering::EBaseLayerInclusion::Exclude);
		if (OriginalLayerIndex == NewLayerIndex || OriginalLayerIndex == INDEX_NONE)
		{
			return;
		}
		
		const FScopedTransaction Transaction(LOCTEXT("MoveLayerToIndexAction", "Move Layer to Index"));
		Modify();

		if (NewLayerIndex > OriginalLayerIndex)
		{
			NewLayerIndex += 1;
		}
		else
		{
			OriginalLayerIndex += 1;
		}
		
		if (NewLayerIndex > Layers.Num())
		{
			Layers.AddZeroed();
		}
		else
		{
			Layers.InsertZeroed(NewLayerIndex);
		}
		
		Layers.Swap(NewLayerIndex, OriginalLayerIndex);
		Layers.RemoveAt(OriginalLayerIndex);
		
		BroadcastLayerStackChanged(EAnimNextEditorDataNotifType::PropertyChanged, true);
	}
}

TObjectPtr<UUAFLayer> UUAFLayerStack_EditorData::AddDefaultAssetBasedLayer(const FAssetData& InAsset)
{
	const FScopedTransaction Transaction(LOCTEXT("AddLayerAction", "Add Layer"));
	Modify();
	
	UUAFLayer* NewLayer = NewObject<UUAFLayer>(this);
	if (!NewLayer)
	{
		UE_LOGF(LogAnimation, Error, "UUAFLayerStack_EditorData::AddDefaultAssetBasedLayer: Failed to create a new layer!");
		return nullptr;
	}
	
	const int32 NewLayerIndex = Layers.Add(NewLayer);
	if (NewLayerIndex == INDEX_NONE)
	{
		UE_LOGF(LogAnimation, Error, " UUAFLayerStack_EditorData::AddDefaultAssetBasedLayer: Failed to add new layer to layer array");
		return nullptr;
	}
	
	FUAFLayerAssetProvider NewContentProvider;
	NewContentProvider.SetLayerAsset(InAsset.GetAsset());
	
	NewLayer->SetLayerContentProvider(TInstancedStruct<FUAFLayerAssetProvider>::Make(NewContentProvider));
	NewLayer->SetLayerBlendProvider(TInstancedStruct<FUAFDefaultBlendProvider>::Make());
	
	OnLayerAdded(NewLayer);
	
	return NewLayer;
}

void UUAFLayerStack_EditorData::BroadcastLayerStackChanged(EAnimNextEditorDataNotifType InType, bool bRefreshUI)
{
	BroadcastModified(InType, this);
	if (bRefreshUI)
	{
		NotifyLayerLayoutChanged();
	}
}

void UUAFLayerStack_EditorData::Initialize(bool bRecompileVM)
{
	Super::Initialize(bRecompileVM);
	
	if (BaseLayer == nullptr)
	{
		BaseLayer = NewObject<UUAFBaseLayer>(this);
		OnLayerAdded(BaseLayer);
	}
}

TSubclassOf<UAssetUserData> UUAFLayerStack_EditorData::GetAssetUserDataClass() const
{
	return UUAFLayerStackWorkspaceAssetUserData::StaticClass();
}

TConstArrayView<TSubclassOf<UUAFRigVMAssetEntry>> UUAFLayerStack_EditorData::GetEntryClasses() const
{
	static const TSubclassOf<UUAFRigVMAssetEntry> Classes[] =
	{
			UAnimNextVariableEntry::StaticClass(),
			UAnimNextAnimationGraphEntry::StaticClass(),
			UUAFSharedVariablesEntry::StaticClass(),
	};

	return Classes;
}

#undef LOCTEXT_NAMESPACE
