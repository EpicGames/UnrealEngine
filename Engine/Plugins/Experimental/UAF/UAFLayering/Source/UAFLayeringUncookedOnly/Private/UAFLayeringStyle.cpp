// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAFLayeringStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateTypes.h"

FUAFLayeringStyle& FUAFLayeringStyle::Get()
{
	static FUAFLayeringStyle Instance;
	return Instance;
}

FUAFLayeringStyle::FUAFLayeringStyle()
	: FSlateStyleSet(TEXT("UAFLayeringStyle"))
{
	SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	Set("Layer.Colors.BlendIndictor", FSlateColor(FColor(112, 214, 255)));
	Set("Layer.Colors.AdditiveIndictor", FSlateColor(FColor(255, 112, 166)));
	Set("Layer.Colors.CacheOnlyIndictor", FSlateColor(FLinearColor(0.1f, 0.1f, 0.1f)));
	
	Set("LayerStack.OuterBorder", new FSlateRoundedBoxBrush(FStyleColors::Panel.GetSpecifiedColor(), 25.0f));
	Set("LayerStack.InnerBorder", new FSlateRoundedBoxBrush(FStyleColors::Background.GetSpecifiedColor(), 10.0f));
	Set("Layer.Background", new FSlateRoundedBoxBrush(FStyleColors::Panel.GetSpecifiedColor(), 10.0f));
	Set("Layer.DisabledBackground", new FSlateRoundedBoxBrush(FStyleColors::Recessed.GetSpecifiedColor(), 10.0f));
	
	Set("Layer.WhiteBackground", new FSlateRoundedBoxBrush(FLinearColor::White, 5.0f));
	Set("Layer.AdvancedSettingsBackground", new FSlateRoundedBoxBrush(FStyleColors::Recessed.GetSpecifiedColor(), FVector4(0.0f, 0.0f, 10.0f, 10.0f))); //. X = Top Left, Y = Top Right, Z = Bottom Right, W = Bottom Left
	Set("Layer.SelectedLayer", new FSlateRoundedBoxBrush(FLinearColor(0.5f, 0.3f, 0.8f), 5.0f));
	Set("Layer.DraggedLayerBackground", new FSlateBoxBrush(NAME_None,FMargin(10.0f, 10.0f), FStyleColors::Recessed.GetSpecifiedColor()));
	
	Set("LayerStack.MiddleBackground", new FSlateRoundedBoxBrush(FStyleColors::AccentGray.GetSpecifiedColor(), 10.0f));
	
	Set("LayerStack.ListView", FTableViewStyle()
	.SetBackgroundBrush(FSlateNoResource()));

	Set("AddLayerButton", FButtonStyle()
	.SetNormal(FSlateNoResource())
	.SetHovered(FSlateRoundedBoxBrush(FLinearColor(1, 1, 1, 0.15f), 4.0f))
	.SetPressed(FSlateRoundedBoxBrush(FLinearColor(1, 1, 1, 0.25f), 4.0f))
	.SetNormalPadding(FMargin(0, 0, 0, 1))
	.SetPressedPadding(FMargin(0, 1, 0, 0)));
	
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon64x64(64.0f, 64.0f);
	const FVector2D Icon32x32(32.0f, 32.0f);
	
	Set("LayerStack.ListViewRow", FTableRowStyle()
		.SetEvenRowBackgroundBrush(FSlateNoResource())
		.SetEvenRowBackgroundHoveredBrush(FSlateNoResource())
		.SetOddRowBackgroundBrush(FSlateNoResource())
		.SetOddRowBackgroundHoveredBrush(FSlateNoResource())
		.SetActiveHighlightedBrush(FSlateNoResource())
		.SetActiveBrush(FSlateRoundedBoxBrush(FStyleColors::AccentBlue.GetSpecifiedColor(), 10.0f))
		.SetActiveHoveredBrush(FSlateRoundedBoxBrush(FStyleColors::AccentBlue.GetSpecifiedColor(), 10.0f))
		.SetSelectorFocusedBrush(FSlateNoResource())
		.SetInactiveHighlightedBrush(FSlateNoResource())
		.SetInactiveBrush(FSlateRoundedBoxBrush(FStyleColors::AccentBlue.GetSpecifiedColor(), 10.0f))
		.SetInactiveHoveredBrush(FSlateRoundedBoxBrush(FStyleColors::AccentBlue.GetSpecifiedColor(), 10.0f))
		.SetDropIndicator_Above(CORE_BOX_BRUSH("Common/DropZoneIndicator_Above", FMargin(10.0f / 16.0f, 10.0f / 16.0f, 0, 0), FStyleColors::AccentOrange.GetSpecifiedColor()))
		.SetDropIndicator_Onto(CORE_BOX_BRUSH("Common/DropZoneIndicator_Onto", FMargin(4.0f / 16.0f), FStyleColors::AccentOrange.GetSpecifiedColor()))
		.SetDropIndicator_Below(CORE_BOX_BRUSH("Common/DropZoneIndicator_Below", FMargin(10.0f / 16.0f, 0, 0, 10.0f / 16.0f), FStyleColors::AccentOrange.GetSpecifiedColor()))
		);
	
	Set("ClassIcon.UAFLayerStack", new IMAGE_BRUSH("Icons/layers_16x", Icon16x16));
	Set("ClassIcon.UAFLayer", new IMAGE_BRUSH("Icons/layer_16x", Icon16x16));
	
	Set("Layer.VisibleLayerIcon", new CORE_IMAGE_BRUSH_SVG("Starship/Common/visible", Icon16x16));
	Set("Layer.DisabledLayerIcon", new IMAGE_BRUSH("Icons/SourceControlOff_16x", Icon16x16));
	Set("Layer.NoPreviewLayerIcon", new CORE_IMAGE_BRUSH_SVG("Starship/Common/hidden", Icon16x16));

	Set("Layer.ExpandAdvancedSettingsIcon", new IMAGE_BRUSH("Common/DownArrow", Icon16x16));
	Set("Layer.CloseAdvancedSettingsIcon", new IMAGE_BRUSH("Common/UpArrow", Icon16x16));

	Set("Layer.AddLayerIcon", new IMAGE_BRUSH("Icons/icon_levels_addlayer_16x", Icon16x16));
	
	Set("Layer.PoseIconBig", new IMAGE_BRUSH_SVG("Starship/AnimationBlueprintEditor/AnimationGraphPose", Icon32x32));
	Set("Layer.PoseIconSmall", new IMAGE_BRUSH_SVG("Starship/AnimationBlueprintEditor/AnimationGraphPose", Icon16x16));
	
	Set("Layer.LODThreshold", new IMAGE_BRUSH("DynamicMaterial/Icons/EditorIcons/MaskToggle_16x", Icon16x16));
	
	Set("Layer.CacheOnlyBackground", new IMAGE_BRUSH("/Graph/GraphPanel_StripesBackground", Icon64x64, FLinearColor(0.12f, 0.12f, 0.12f, 0.3f), ESlateBrushTileType::Both ));
	Set("Layer.AddLayerBackground", new BORDER_BRUSH("/Icons/UMG/ToggleOutlines",  FMargin(4.0f/16.0f), FLinearColor::White));
	
	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FUAFLayeringStyle::~FUAFLayeringStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}
