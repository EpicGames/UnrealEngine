// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAFLayeringUncookedOnlyModule.h"

#define LOCTEXT_NAMESPACE "FUAFLayeringEditorModule"

namespace UE::UAF::Layering
{
void FUAFLayeringUncookedOnlyModule::StartupModule()
{
}

void FUAFLayeringUncookedOnlyModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FUAFLayeringUncookedOnlyModule, UAFLayeringUncookedOnly)
}

#undef LOCTEXT_NAMESPACE
	
