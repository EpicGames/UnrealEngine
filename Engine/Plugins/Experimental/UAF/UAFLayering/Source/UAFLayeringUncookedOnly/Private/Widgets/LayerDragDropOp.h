// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAFLayeringStyle.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Layers/UAFLayer.h"
#include "Widgets/SOverlay.h"

class UUAFLayer;

#define LOCTEXT_NAMESPACE "FLayerDragDropOp"

namespace UE::UAF::Layering
{
	class FLayerDragDropOp : public FDecoratedDragDropOp
	{
	public:
		DRAG_DROP_OPERATOR_TYPE(FLayerDragDropOp, FDecoratedDragDropOp)
		
		static TSharedRef<FLayerDragDropOp> New(const TWeakObjectPtr<UUAFLayer> InLayer)
		{
			TSharedRef<FLayerDragDropOp> Operation = MakeShared<FLayerDragDropOp>();
			Operation->DraggedLayer = InLayer;

			Operation->Construct();

			return Operation;
		}
		
		virtual TSharedPtr<SWidget> GetDefaultDecorator() const override
		{
			return SNew(SOverlay)
				+ SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SNew(SBorder)
					.BorderImage(FUAFLayeringStyle::Get().GetBrush(TEXT("Layer.DraggedLayerBackground")))
					.Padding(10.0f)
						[
					
						SNew(SBorder)
						.BorderImage(FUAFLayeringStyle::Get().GetBrush(TEXT("Layer.Background")))
						[
							SNew(SVerticalBox)
							+ SVerticalBox::Slot()
							.Padding(20.0f, 10.0f)
							[
								SNew(STextBlock)
								.Text_Lambda([this]()
									{
										return DraggedLayer.IsValid() ? FText::FromName(DraggedLayer->GetLayerName()) : LOCTEXT("InvalidDraggedLayerLabel", "Invalid Layer");
									})
							]
						]
					]
				];	
		}
		
		TWeakObjectPtr<UUAFLayer> DraggedLayer = nullptr;
	};
}

#undef LOCTEXT_NAMESPACE

