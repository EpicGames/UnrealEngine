// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SDefaultBlendProvider.h"

#include "Editor.h"
#include "ISinglePropertyView.h"
#include "ScopedTransaction.h"
#include "SlateOptMacros.h"
#include "Layers/UAFLayer.h"
#include "Layers/UAFLayerDefaultBlendProvider.h"
#include "Widgets/Input/SEditableText.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "UAFDefaultBlendProvider"

namespace UE::UAF::Layering
{
void SDefaultBlendProvider::Construct(const FArguments& InArgs)
{
	WeakLayer = InArgs._Layer;
	if (!ensureMsgf(WeakLayer.IsValid(), TEXT("Tried to create a SDefaultBlendProvider without a valid Layer")))
	{
		return;
	}
	
	FSinglePropertyParams DefaultPropertyArgs;
	DefaultPropertyArgs.NamePlacement = EPropertyNamePlacement::Hidden;
	DefaultPropertyArgs.bHideResetToDefault = true;
	DefaultPropertyArgs.bHideAssetThumbnail = true;
	DefaultPropertyArgs.NotifyHook = this;
	
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	TSharedRef<FDefaultLayerBlendStructureDataProvider> LayerBlendDataProvider = MakeShared<FDefaultLayerBlendStructureDataProvider>(&WeakLayer->GetLayerBlendProvider());
	
	// Generate single property views to display blend properties within the layer widget 
	TSharedPtr<ISinglePropertyView> LayerBlendModePropertyView = PropertyEditorModule.CreateSingleProperty(LayerBlendDataProvider, GET_MEMBER_NAME_CHECKED(FUAFDefaultBlendProvider, BlendMode), DefaultPropertyArgs);
	TSharedPtr<ISinglePropertyView> LayerWeightPropertyView =  PropertyEditorModule.CreateSingleProperty(LayerBlendDataProvider, GET_MEMBER_NAME_CHECKED(FUAFDefaultBlendProvider, LayerWeight), DefaultPropertyArgs);
	if (LayerWeightPropertyView)
	{
		LayerWeightPropertyView->SetToolTipText(LOCTEXT("LayerWeightTooltip", "The weight to use to apply this layer. 0 == this layer won't get applied at all, 1 == 100% of the layer gets applied"));
	}
	
	TSharedPtr<ISinglePropertyView> LayerMaskPropertyView =  PropertyEditorModule.CreateSingleProperty(LayerBlendDataProvider, GET_MEMBER_NAME_CHECKED(FUAFDefaultBlendProvider, BlendMask), DefaultPropertyArgs);
	if (LayerMaskPropertyView)
	{
		LayerMaskPropertyView->SetToolTipText(LOCTEXT("BlendMaskToolTip", "An optional blend mask to apply when blending this layers pose result with the previous layers pose"));
	}
	
	TSharedPtr<ISinglePropertyView> BlendCurvePropertyView =  PropertyEditorModule.CreateSingleProperty(LayerBlendDataProvider, GET_MEMBER_NAME_CHECKED(FUAFDefaultBlendProvider, BlendCurve), DefaultPropertyArgs);
	if (BlendCurvePropertyView)
	{
		BlendCurvePropertyView->SetToolTipText(LOCTEXT("BlendCurveToolTip", "The custom blend curve to use when blending this layer in or out"));
	}
	
	// Display the name for the following property views
	DefaultPropertyArgs.NamePlacement = EPropertyNamePlacement::Left;
	TSharedPtr<ISinglePropertyView> LayerEnabledPropertyView =  PropertyEditorModule.CreateSingleProperty(LayerBlendDataProvider, GET_MEMBER_NAME_CHECKED(FUAFDefaultBlendProvider, bLayerEnabled), DefaultPropertyArgs);
	TSharedPtr<ISinglePropertyView> LayerBlendInTimePropertyView =  PropertyEditorModule.CreateSingleProperty(LayerBlendDataProvider, GET_MEMBER_NAME_CHECKED(FUAFDefaultBlendProvider, LayerBlendInTime), DefaultPropertyArgs);
	TSharedPtr<ISinglePropertyView> LayerBlendOutTimePropertyView =  PropertyEditorModule.CreateSingleProperty(LayerBlendDataProvider, GET_MEMBER_NAME_CHECKED(FUAFDefaultBlendProvider, LayerBlendOutTime), DefaultPropertyArgs);

	ChildSlot
	[
		SNew(SSplitter)
		.Orientation(Orient_Horizontal)
		.PhysicalSplitterHandleSize(4.0f)
		.HitDetectionSplitterHandleSize(6.0f)
		+ SSplitter::Slot()
		.Value(0.3f)
		[
			SNew(SBox)
			.Padding(10.0f)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.Padding(5.0f, 2.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("BlendModeLabel", "Blend Mode"))
					.ToolTipText(LOCTEXT("BlendModeLabelToolTip", "The blend mode this layer uses to blend its result with the pose result of the previous layer"))
				]
				+ SVerticalBox::Slot()
				.Padding(0.0f, 2.0f)
				[
					LayerBlendModePropertyView ? LayerBlendModePropertyView.ToSharedRef() : SNullWidget::NullWidget
				]
				+ SVerticalBox::Slot()
				.Padding(0.0f, 2.0f)
				[
					LayerMaskPropertyView ? LayerMaskPropertyView.ToSharedRef() : SNullWidget::NullWidget
				]
			]

		]
		+ SSplitter::Slot()
		.Value(0.3f)
		[
			SNew(SBox)
			.Padding(10.0f)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.Padding(5.0f, 2.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("WeightLabel", "Weight"))
					.ToolTipText(LOCTEXT("LayerWeightLabelTooltip", "The weight to use to apply this layer. 0 == this layer won't get applied at all, 1 == 100% of the layer gets applied"))
				]
				+ SVerticalBox::Slot()
				.Padding(0.0f, 2.0f)
				[
					LayerWeightPropertyView ? LayerWeightPropertyView.ToSharedRef() : SNullWidget::NullWidget
				]
				+ SVerticalBox::Slot()
				.Padding(5.0f, 2.0f)
				[
					SNew(SEditableTextBox)
					.Text(FText::FromString(TEXT("Layer Debug Weight")))
					.IsEnabled(false)
				]
			]

		]
		+ SSplitter::Slot()
		.Value(0.4f)
		[
			SNew(SBox)
			.Padding(10.0f)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.Padding(5.0f, 2.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("BlendSettingLabel", "Blend Settings"))
					.ToolTipText(LOCTEXT("BlendSettingLabelTooltip", "These settings drive the discrete blend when a layer gets dynamically enabled or disabled during runtime"))
				]
				+ SVerticalBox::Slot()
				.Padding(0.0f, 2.0f)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					[
						LayerEnabledPropertyView ? LayerEnabledPropertyView.ToSharedRef() : SNullWidget::NullWidget
					]
					+ SHorizontalBox::Slot()
					[
						BlendCurvePropertyView ? BlendCurvePropertyView.ToSharedRef() : SNullWidget::NullWidget
					]

				]
				+ SVerticalBox::Slot()
				.Padding(0.0f, 2.0f)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					[
						LayerBlendInTimePropertyView ? LayerBlendInTimePropertyView.ToSharedRef() : SNullWidget::NullWidget
					]
					+ SHorizontalBox::Slot()
					[
						LayerBlendOutTimePropertyView ? LayerBlendOutTimePropertyView.ToSharedRef() : SNullWidget::NullWidget
					]

				]
			]

		]
	];
}

void SDefaultBlendProvider::NotifyPreChange(class FEditPropertyChain* PropertyAboutToChange)
{
	if (WeakLayer.IsValid())
	{
		WeakLayer->Modify();
	}
}

void SDefaultBlendProvider::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, class FEditPropertyChain* PropertyThatChanged)
{
	if (WeakLayer.IsValid())
	{
		WeakLayer->BroadcastModified(EAnimNextEditorDataNotifType::PropertyChanged);
	}
}
}
#undef LOCTEXT_NAMESPACE //UAFDefaultBlendProvider
