// Copyright Epic Games, Inc. All Rights Reserved.


#include "Widgets/SLayer.h"

#include "SlateOptMacros.h"
#include "UAFLayeringStyle.h"
#include "Components/VerticalBox.h"
#include "Layers/UAFLayer.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "FUAFLayerWidget"

namespace UE::UAF::Layering
{
	
void SLayer::Construct(const FArguments& InArgs)
{
	Layer = InArgs._Layer;
	ensure(Layer.IsValid());

	ChildSlot
	[
		SNew(SOverlay)
		+ SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			SNew(SBorder)
			.BorderImage(this, &SLayer::GetBackgroundBrush)
			[
				SNew(SBorder)
				.BorderImage(this, &SLayer::GetBackgroundOverlayBrush)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.MaxWidth(5.0f)
						.Padding(10.0f)
						[
							SNew(SBorder)
							.BorderImage(FUAFLayeringStyle::Get().GetBrush(TEXT("Layer.WhiteBackground")))
							.BorderBackgroundColor(this, &SLayer::GetIndicatorColor)
							.ToolTipText(LOCTEXT("ColorIndicatorTooltip", "The color indicator for this layer, signifying what operations this layer performs"))
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Fill)
						.VAlign(VAlign_Fill)
						[
							// Grabber Button for drag and drop
							SNew(SBorder)
							.BorderImage( FAppStyle::GetBrush("NoBorder") )
							.Cursor(EMouseCursor::GrabHand)
							.HAlign(HAlign_Center)
							.VAlign(VAlign_Center)
							.Padding(8.0f)
							[
								SNew(SImage)
								.Image(FAppStyle::GetBrush("VerticalBoxDragIndicatorShort"))
								
							]
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						[
							// Visibility button 
							SNew(SButton)
							.ContentPadding(FMargin(5.0f))
							.ButtonStyle(FAppStyle::Get(), "NoBorder")
							.ForegroundColor(FSlateColor::UseForeground())
							.HAlign(HAlign_Center)
							.VAlign(VAlign_Center)
							.ToolTipText_Lambda([this]() -> FText
								{
									if (Layer.IsValid())
									{
										switch (Layer->GetLayerState())
										{
										case EUAFLayerState::Enabled:
											return LOCTEXT("LayerVisibilityTooltip_DisablePreview", "Disable this layer for preview. It will still be enabled at runtime.");
										case EUAFLayerState::PreviewDisabled:
											return LOCTEXT("LayerVisibilityTooltip_EnablePreview", "Enable this layer for preview.");
										case EUAFLayerState::Disabled:
											return LOCTEXT("LayerVisibilityTooltip_Disabled", "This layer is currently disabled for preview and runtime.");
										default:
											return FText();
										}
									}
									return FText();
								})
							.OnClicked_Lambda([this]() -> FReply
								{
									if (Layer.IsValid())
									{
										Layer->TogglePreviewVisibility();
									}
									return FReply::Handled();
								})
							.Content()
							[
								SNew(SImage)
								.Image(this, &SLayer::GetVisibilityBrushForLayer)
							]
						]
						+ SHorizontalBox::Slot()
						.Padding(5.0f, 10.f)
						.FillContentWidth(1.0f)
						[
							SNew(SSplitter)
							.Orientation(Orient_Horizontal)
							.PhysicalSplitterHandleSize(4.0f)
							.HitDetectionSplitterHandleSize(6.0f)
							+ SSplitter::Slot()
							.Value(0.2)
							[
								SNew(SVerticalBox)
								+ SVerticalBox::Slot()
								.AutoHeight()
								.Padding(10.0f, 10.0f, 0.0f, 0.0f)
								[
									SNew(SInlineEditableTextBlock)
									.OnTextCommitted(this, &SLayer::OnLayerNameCommitted)
									.ToolTipText(LOCTEXT("LayerNameTooltip", "The name of this layer. The name has to be unique within the layer stack"))
									.Text(this, &SLayer::GetLayerName)
									.IsSelected(this, &SLayer::IsSelected)
									.OnVerifyTextChanged(this, &SLayer::VerifyLayerName)
								]
								+ SVerticalBox::Slot()
								.AutoHeight()
								.Padding(10.0f, 5.0f)
								[
									Layer.IsValid() && Layer->IsLayerContentProviderValid()
									? Layer->GetLayerContentProvider().GetMutable().CreateLayerContentWidget(Layer.Get())
									: SNullWidget::NullWidget
								]
								+ SVerticalBox::Slot()
								.AutoHeight()
								.Padding(10.0f, 0.0f)
								[
									SNew(SHorizontalBox)
									+ SHorizontalBox::Slot()
									.AutoWidth()
									[
										SNew(SImage)
										.Image(FUAFLayeringStyle::Get().GetBrush(TEXT("Layer.PoseIconSmall")))
										.ColorAndOpacity(FLinearColor::Yellow)
									]
								]
							]
							+ SSplitter::Slot()
							.Value(0.8)
							[
								Layer.IsValid() && Layer->IsLayerBlendProviderValid()
								? Layer->GetLayerBlendProvider().GetMutable().CreateLayerBlendWidget(Layer.Get())
								: SNullWidget::NullWidget
							]
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Right)
						.Padding(20.0f, 0.0f, 0.0f, 0.0f)
						[
							// Advanced Settings button 
							SNew(SButton)
							.ContentPadding(FMargin(5.0f, 5.0f, 20.0f, 5.0f))
							.ButtonStyle(FAppStyle::Get(), "NoBorder")
							.ForegroundColor(FSlateColor::UseForeground())
							.HAlign(HAlign_Center)
							.VAlign(VAlign_Center)
							.ToolTipText(LOCTEXT("AdvancedLayerSettingsToolTip", "Advanced settings for this layer"))
							.OnClicked_Lambda([this]() -> FReply
								{
									bAdvancedSettingsExpanded = !bAdvancedSettingsExpanded;
									return FReply::Handled();
								})
							.Content()
							[
								SNew(SImage)
								.Image(this, &SLayer::GetAdvancedSettingBrushForLayer)
							]
						]

					]
					+ SVerticalBox::Slot() // Advanced Settings Container 
					.AutoHeight()
					[
						SNew(SBorder)
						.BorderImage(FUAFLayeringStyle::Get().GetBrush(TEXT("Layer.AdvancedSettingsBackground")))
						.Padding(10.0f)
						.Visibility_Lambda([this]() -> EVisibility
							{
								return bAdvancedSettingsExpanded
									? EVisibility::Visible
									: EVisibility::Collapsed;
							})
						[
							SNew(SSplitter)
							.Orientation(Orient_Horizontal)
							.PhysicalSplitterHandleSize(4.0f)
							.HitDetectionSplitterHandleSize(6.0f)
							+ SSplitter::Slot() // Cached Output Pose
							.Value(0.2)
							[
								SNew(SVerticalBox)
								+ SVerticalBox::Slot()
								.Padding(20.0f, 10.0f)
								.AutoHeight()
								[
									SNew(STextBlock)
									.Text(LOCTEXT("OutputPoseLabel", "Output Pose"))
									.ToolTipText(LOCTEXT("OutputPoseLabelToolTip", "The optionally cached pose output of this layer"))
								]
								+ SVerticalBox::Slot()
								.Padding(20.0f, 0.0f)
								.AutoHeight()
								[
									SNew(SHorizontalBox)
									+ SHorizontalBox::Slot()
									.Padding(0.0f, 2.0f)
									.AutoWidth()
									[
										SNew(SImage)
										.Image(FUAFLayeringStyle::Get().GetBrush(TEXT("Layer.PoseIconSmall")))
										.ColorAndOpacity(FLinearColor::Yellow)
									]
									+ SHorizontalBox::Slot()
									.AutoWidth()
									.Padding(5.0f, 0.0f)
									[
										SNew(SEditableTextBox)
										.Text(FText::FromString(TEXT("Output Pose Binding")))
										.IsEnabled(false)
										.ToolTipText(LOCTEXT("OutputPoseBindingToolTip", "The property binding the cached output pose of this layer is written to"))
									]
								]
							]
							+ SSplitter::Slot() // Cached Additive Pose
							.Value(0.4f)
							[
								SNew(SHorizontalBox) 
								+ SHorizontalBox::Slot()
								.Padding(20.0f, 10.0f, 5.0f, 0.0f)
								.AutoWidth()
								.HAlign(HAlign_Left)
								[
									SNew(SVerticalBox)
									+ SVerticalBox::Slot()
									.HAlign(HAlign_Right)
									.AutoHeight()
									.Padding(0.0f, 0.0f, 0.0f, 5.0f)
									[
										SNew(SHorizontalBox)
										+ SHorizontalBox::Slot()
										.AutoWidth()
										[
											SNew(STextBlock)
											.Text(LOCTEXT("AdditiveOutputPoseBindingLabel", "Output Additive Pose"))
											.ToolTipText(LOCTEXT("AdditiveOutputPoseLabelToolTip", "The optionally cached additive pose output of this layer and a base"))
										]
										+ SHorizontalBox::Slot()
										.AutoWidth()
										.Padding(5.0f, 0.0f)
										[
											SNew(SImage)
											.Image(FUAFLayeringStyle::Get().GetBrush(TEXT("Layer.PoseIconSmall"))) // TODO: Change to additive icon
											.ColorAndOpacity(FLinearColor::Yellow)
										]
									]
									+ SVerticalBox::Slot()
									.Padding(0.0f, 5.0f, 0.0f, 5.0f)
									.HAlign(HAlign_Right)
									[
										SNew(STextBlock)
										.Text(LOCTEXT("AdditiveBaseLabel", "Base"))
										.ToolTipText(LOCTEXT("AdditiveBaseLabelToolTip", "The base pose for the additive calculation of this layers optional additive output pose"))
									]
								]
								+ SHorizontalBox::Slot()
								.HAlign(HAlign_Left)
								.AutoWidth()
								[
									SNew(SVerticalBox)
									+ SVerticalBox::Slot()
									.AutoHeight()
									.Padding(5.0f)
									[
										// Additive binding
										SNew(SEditableTextBox)
										.Text(FText::FromString(TEXT("Additive Pose Binding")))
										.IsEnabled(false)
										.ToolTipText(LOCTEXT("AdditivePoseBindingToolTip", "The property binding the cached additive output pose of this layer is written to"))
									]
									+ SVerticalBox::Slot()
									.Padding(5.0f)
									.AutoHeight()
									[
										SNew(SHorizontalBox)
										+ SHorizontalBox::Slot()
										.AutoWidth()
										[
											SNew(SEditableTextBox)
											.Text(FText::FromString(TEXT("Base For Additive")))
											.IsEnabled(false)
										]
										+ SHorizontalBox::Slot()
										.AutoWidth()
										.Padding(5.0f, 0.0f, 5.0f, 0.0f)
										[
											SNew(STextBlock)
											.Text(LOCTEXT("AdditiveLabel", "Additive"))
										]
										+ SHorizontalBox::Slot()
										.AutoWidth()
										[
											SNew(SEditableTextBox)
											.Text(FText::FromString(TEXT("Additive For Additive")))
											.IsEnabled(false)
										]
									]
								]
							]
							+ SSplitter::Slot() // LOD Threshold
							.Value(0.4f)
							[
								SNew(SVerticalBox)
								+ SVerticalBox::Slot()
								.Padding(20.0f, 5.0f)
								.AutoHeight()
								[
									SNew(STextBlock)
									.Text(LOCTEXT("LODSettingLabel", "LOD Settings"))
								]
								+ SVerticalBox::Slot()
								.Padding(20.0f, 5.0f)
								.AutoHeight()
								[
									SNew(SHorizontalBox)
									+ SHorizontalBox::Slot()
									.AutoWidth()
									[
										// Icon
										SNew(SImage)
										.Image(FUAFLayeringStyle::Get().GetBrush(TEXT("Layer.LODThreshold")))
									]
									+ SHorizontalBox::Slot()
									.Padding(5.0f, 0.0f)
									.AutoWidth()
									[
										// Binding
										SNew(SEditableTextBox)
										.Text(FText::FromString(TEXT("Mask Binding")))
										.IsEnabled(false)
									]
								]
							]
						]
					]
				]
			]
		]
	];

}

const FSlateBrush* SLayer::GetVisibilityBrushForLayer() const
{
	if (Layer.IsValid())
	{
		switch (Layer->GetLayerState())
		{
		case EUAFLayerState::Enabled:
			return FUAFLayeringStyle::Get().GetBrush(TEXT("Layer.VisibleLayerIcon"));
		case EUAFLayerState::PreviewDisabled: 
			return FUAFLayeringStyle::Get().GetBrush(TEXT("Layer.NoPreviewLayerIcon"));
		case EUAFLayerState::Disabled:
			return FUAFLayeringStyle::Get().GetBrush(TEXT("Layer.DisabledLayerIcon"));
		default:
			return FUAFLayeringStyle::Get().GetBrush(TEXT("Layer.VisibleLayerIcon"));
		}
	}
	return FUAFLayeringStyle::Get().GetBrush(TEXT("Layer.VisibleLayerIcon"));
}

const FSlateBrush* SLayer::GetAdvancedSettingBrushForLayer() const
{
	if (bAdvancedSettingsExpanded)
	{
		return FUAFLayeringStyle::Get().GetBrush(TEXT("Layer.CloseAdvancedSettingsIcon"));
	}

	return FUAFLayeringStyle::Get().GetBrush(TEXT("Layer.ExpandAdvancedSettingsIcon"));
}

const FSlateBrush* SLayer::GetBackgroundBrush() const
{
	return  Layer.IsValid() && Layer->GetLayerState() == EUAFLayerState::Disabled ? FUAFLayeringStyle::Get().GetBrush(TEXT("Layer.DisabledBackground")) : FUAFLayeringStyle::Get().GetBrush(TEXT("Layer.Background"));
}

const FSlateBrush* SLayer::GetBackgroundOverlayBrush() const
{
	// Ask providers for overrides 
	if (Layer.IsValid() && Layer->IsLayerBlendProviderValid())
	{
		if (const FSlateBrush* OverrideBrush = Layer->GetLayerBlendProvider().Get().GetOverrideLayerBackground())
		{
			return OverrideBrush;
		}
	}

	return FAppStyle::GetNoBrush();
}

void SLayer::OnLayerNameCommitted(const FText& NewText, ETextCommit::Type CommitInfo) const
{
	if (Layer.IsValid())
	{
		Layer->RenameLayer(FName(NewText.ToString()));
	}
}

FSlateColor SLayer::GetIndicatorColor() const
{
	if (Layer.IsValid() && Layer->IsLayerBlendProviderValid())
	{
		FSlateColor OverrideColor;
		if (Layer->GetLayerBlendProvider().Get().GetOverrideIndicatorColor(OverrideColor))
		{
			return OverrideColor;
		}
	}

	return FSlateColor(FLinearColor::White);
}
	
FText SLayer::GetLayerName() const
{
	if (Layer.IsValid())
	{
		return FText::FromName(Layer->GetLayerName());
	}
	return FText();
}

bool SLayer::IsSelected() const
{
	if (TSharedPtr<STableRow<TObjectPtr<UUAFLayer>>> OwningRow = StaticCastSharedPtr<STableRow<TObjectPtr<UUAFLayer>>>(this->GetParentWidget()))
	{
		return OwningRow->IsSelected();
	}

	return false;
}

bool SLayer::VerifyLayerName(const FText& Text, FText& OutErrorMessage) const
{
	if (Layer.IsValid())
	{
		if (const bool bValidName = Layer->IsLayerNameValid(FName(Text.ToString())))
		{
			return true;
		}
		
		OutErrorMessage = FText::FromString(TEXT("Invalid layer name. Layer names have to be unique within the layer stack and not none."));
	}
	
	return false;
}
	
}

#undef LOCTEXT_NAMESPACE

