// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SLayerStack.h"

#include "AnimNextAnimGraphSettings.h"
#include "UAFLayerStack.h"
#include "IWorkspaceEditor.h"
#include "LayerDragDropOp.h"
#include "UncookedOnlyUtils.h"
#include "UAFLayerStack_EditorData.h"
#include "Layers/UAFLayer.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SBoxPanel.h"
#include "UAFLayeringStyle.h"
#include "SAssetDropTarget.h"
#include "LayeringUncookedOnlyTypes.h"
#include "ScopedTransaction.h"
#include "Layers/UAFBaseLayer.h"

#define LOCTEXT_NAMESPACE "FUAFLayerStackWidget"

namespace UE::UAF::Layering
{

void SLayerStack::Construct(const FArguments& InArgs)
{
	// args
	LayerStack = InArgs._LayerStack;
	WeakWorkspaceEditor = InArgs._WorkspaceEditor;
	ensure(LayerStack.IsValid());

	UUAFLayerStack_EditorData* EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData<UUAFLayerStack_EditorData>(LayerStack.Get());
	EditorData->OnLayerLayoutChanged.AddSP(this, &SLayerStack::OnLayerLayoutChanged);
	EditorData->OnLayerSelectionChanged.AddSP(this, &SLayerStack::OnLayerSelectionChanged);
	
	CacheLayersForList();

	// make commands for context menu
	CommandList = MakeShareable(new FUICommandList);

	// widget content 
	ChildSlot
	[
		SNew(SBorder)
		.Padding(10.0f)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.BorderImage(FUAFLayeringStyle::Get().GetBrush(TEXT("LayerStack.OuterBorder")))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.FillContentHeight(1.0f)
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SNew(SBorder)
					.BorderImage(FUAFLayeringStyle::Get().GetBrush(TEXT("LayerStack.InnerBorder")))
				]
				+ SOverlay::Slot()
				.Padding(10.0f)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					[
						SAssignNew(LayerListView, SListView<TWeakObjectPtr<UUAFLayer>>)
						.ListViewStyle(FUAFLayeringStyle::Get(), "LayerStack.ListView")
						.ListItemsSource(&CachedLayerPointers)
						.OnGenerateRow(this, &SLayerStack::GenerateLayerRow)
						.ClearSelectionOnClick(true)
						.SelectionMode(ESelectionMode::Single)
						.OnSelectionChanged(this, &SLayerStack::OnListSelectionChanged)
						.OnContextMenuOpening(this, &SLayerStack::OnConstructContextMenu)
					]
					+ SVerticalBox::Slot()
					.MinHeight(50.0f)
					.AutoHeight()
					.VAlign(VAlign_Top)
					.Padding(20.0f)
					[
						SNew(SAssetDropTarget)
						.bSupportsMultiDrop(false)
						.OnAreAssetsAcceptableForDrop_Lambda([](TArrayView<FAssetData> InAssets)
							{
								const TArray<UClass*> AllowedClasses = UAnimNextAnimGraphSettings::GetAllowedAssetClasses();
								for (const FAssetData& AssetData : InAssets)
								{
									if (AllowedClasses.Contains(AssetData.GetClass()))
									{
										return true;
									}
								}
								return false;
							})
						.OnAssetsDropped_Lambda([EditorData](const FDragDropEvent&, TArrayView<FAssetData> InAssets)
							{
								const TArray<UClass*> AllowedClasses = UAnimNextAnimGraphSettings::GetAllowedAssetClasses();
								for (const FAssetData& AssetData : InAssets)
								{
									if (AllowedClasses.Contains(AssetData.GetClass()))
									{
										EditorData->AddDefaultAssetBasedLayer(AssetData);
									}
								}
							})
						[
							SNew(SBorder)
							.VAlign(VAlign_Fill)
							.HAlign(HAlign_Fill)
							.BorderImage(FUAFLayeringStyle::Get().GetBrush(TEXT("Layer.AddLayerBackground")))
							.ToolTipText(LOCTEXT("AddLayerTooltip", "Adds a new layer"))
							[
								SNew(SButton)
								.ContentPadding(FMargin(10.0F))
								.ButtonStyle(FUAFLayeringStyle::Get(), "AddLayerButton")
								.ForegroundColor(FSlateColor::UseForeground())
								.HAlign(HAlign_Center)
								.VAlign(VAlign_Center)
								.OnReleased_Lambda([EditorData]()
									{
										EditorData->AddDefaultAssetBasedLayer(FAssetData());
									})
								.Content()
								[
									SNew(SHorizontalBox)
									+ SHorizontalBox::Slot()
									.Padding(5.0f)
									.AutoWidth()
									[
										SNew(SImage)
										.Image(FUAFLayeringStyle::Get().GetBrush(TEXT("Layer.AddLayerIcon")))
									]
									+ SHorizontalBox::Slot()
									.Padding(5.0f)
									.AutoWidth()
									[
										SNew(STextBlock)
										.Text(LOCTEXT("AddLayerLabel", "Add Layer"))
									]

								]
							]
						]
					]
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Bottom)
			.Padding(0.0f, 20.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SBorder)
					.Padding(10.0f)
					.BorderImage(FUAFLayeringStyle::Get().GetBrush(TEXT("LayerStack.InnerBorder")))
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SBorder)
							.VAlign(VAlign_Center)
							.HAlign(HAlign_Center)
							.Padding(10.0f)
							.BorderImage(FUAFLayeringStyle::Get().GetBrush(TEXT("Layer.Background")))
							[
								SNew(SImage)
								.Image(FUAFLayeringStyle::Get().GetBrush(TEXT("Layer.PoseIconBig")))
							]
						]
						+ SHorizontalBox::Slot()
						.Padding(10.0f, 10.0f)
						[
							SNew(SVerticalBox)
							+ SVerticalBox::Slot()
							[
								SNew(STextBlock)
								.Text(LOCTEXT("FinalCachedPoseLabel", "Final Cached Pose"))
							]
						]
					]
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Bottom)
				.HAlign(HAlign_Right)
				[
					SNew(SButton)
					.Text(FText::FromString(TEXT("Show Graph")))
					.OnClicked_Lambda([EditorData]()
						{
							if (EditorData->CreatedGraph.IsValid())
							{
								UE::UAF::UncookedOnly::FUtils::OpenProgrammaticGraphs(EditorData, {EditorData->CreatedGraph.Get()});
							}

							return FReply::Handled();
						})
				]
			]
		]
	];
}

SLayerStack::~SLayerStack()
{
	LayerListView->ClearItemsSource();
	CachedLayerPointers.Empty();
}

void SLayerStack::OnListSelectionChanged(TWeakObjectPtr<UUAFLayer> SelectedItem, ESelectInfo::Type SelectInfo) const
{
	if (TSharedPtr<UE::Workspace::IWorkspaceEditor> SharedWorkspaceEditor = WeakWorkspaceEditor.Pin())
	{
		if (SelectedItem.IsValid())
		{
			SharedWorkspaceEditor->SetDetailsObjects({ SelectedItem.Get() });
		}
		else
		{
			SharedWorkspaceEditor->SetDetailsObjects({ LayerStack.Get() });
			LayerListView->ClearSelection();
		}
	}
}

TSharedRef<ITableRow> SLayerStack::GenerateLayerRow(TWeakObjectPtr<UUAFLayer> Item, const TSharedRef<STableViewBase>& OwnerTable) const
{
	return SNew(STableRow<TWeakObjectPtr<UUAFLayer>>, OwnerTable)
		.Padding(3.0f)
		.Style(FUAFLayeringStyle::Get(), "LayerStack.ListViewRow")
		.ShowWires(true)
		.ShowSelection(true)
		.OnDragDetected_Lambda([Item](const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
			{
				if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
				{
					// Check if this is a base layer, we won't allow drag and drop operations on these 
					if (Item.IsValid() && !Item->IsA<UUAFBaseLayer>())
					{
						const TSharedRef<FLayerDragDropOp> DragDropOp = FLayerDragDropOp::New(Item);
						return FReply::Handled().BeginDragDrop(DragDropOp);
					}
				}

				return FReply::Unhandled();
			})
		.OnCanAcceptDrop_Lambda([](const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TWeakObjectPtr<UUAFLayer> TargetItem) -> TOptional<EItemDropZone>
			{
				TOptional<EItemDropZone> ReturnedDropZone = DropZone;
				const TSharedPtr<FLayerDragDropOp> LayerDragDropOp = DragDropEvent.GetOperationAs<FLayerDragDropOp>();
				if (LayerDragDropOp.IsValid())
				{
					// If the target is a base layer, we can only drop below it 
					if (TargetItem.IsValid() && TargetItem->IsA<UUAFBaseLayer>())
					{
						ReturnedDropZone = EItemDropZone::BelowItem;
					}
				}

				return ReturnedDropZone;
			})
		.OnAcceptDrop_Lambda([this](const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TWeakObjectPtr<UUAFLayer> TargetItem) -> FReply
			{
				const TSharedPtr<FLayerDragDropOp> LayerDragDropOp = DragDropEvent.GetOperationAs<FLayerDragDropOp>();
				if (LayerDragDropOp.IsValid() && LayerStack.IsValid() && LayerDragDropOp->DraggedLayer.IsValid())
				{
					UUAFLayerStack_EditorData* EditorData = UAF::UncookedOnly::FUtils::GetEditorData<UUAFLayerStack_EditorData>(LayerStack.Get());
					if (EditorData)
					{
						int32 TargetIndex = EditorData->GetIndexForLayer(TargetItem.Get(), EBaseLayerInclusion::Exclude);
						if (DropZone == EItemDropZone::BelowItem)
						{
							TargetIndex += 1;
						}
					
						EditorData->MoveLayerToIndex(LayerDragDropOp->DraggedLayer.Get(), TargetIndex);
						
						return FReply::Handled();
					}
				}
				return FReply::Unhandled();
			})
		[
			Item.IsValid() ? Item->CreateLayerWidget() : SNullWidget::NullWidget
		];
}

TSharedPtr<SWidget> SLayerStack::OnConstructContextMenu() const
{
	TArray<TWeakObjectPtr<UUAFLayer>> SelectedLayers = LayerListView->GetSelectedItems();
	if (SelectedLayers.Num() == 0 || !SelectedLayers[0].IsValid())
	{
		return SNullWidget::NullWidget;
	}
	
	UUAFLayerStack_EditorData* EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData<UUAFLayerStack_EditorData>(LayerStack.Get());
	if (!EditorData)
	{
		return SNullWidget::NullWidget;
	}
	
	UUAFLayer* SelectedLayer = SelectedLayers[0].Get();
	const int32 LayerIndex = EditorData->GetIndexForLayer(SelectedLayer);
	const int32 NumLayers = EditorData->GetNumLayers();
	const bool bIsBaseLayer = EditorData->IsBaseLayer(SelectedLayer);
	
	constexpr bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, CommandList);
	MenuBuilder.BeginSection("LayerStackLayout", LOCTEXT("LayerStackLayoutLabel", "Layer Stack Layout"));
	{
		const bool bCanMoveLayerUp = LayerIndex > 1;
		const bool bCanMoveLayerDown = LayerIndex > 0 && LayerIndex < NumLayers - 1;

		if (bCanMoveLayerUp)
		{
			MenuBuilder.AddMenuEntry(
			LOCTEXT("MoveLayerUpLabel", "Move Layer Up"),
			LOCTEXT("MoveLayerUpTooltip", "Moves layer up in the execution order."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([EditorData, SelectedLayer, this]()
				{
					EditorData->MoveLayerUp(SelectedLayer);
				})),
			NAME_None,
			EUserInterfaceActionType::Button);
		}
		
		if (bCanMoveLayerDown)
		{
			MenuBuilder.AddMenuEntry(
			LOCTEXT("MoveLayerDownLabel", "Move Layer Down"),
			LOCTEXT("MoveLayerDownTooltip", "Moves layer down in the execution order."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([EditorData, SelectedLayer, this]()
				{
					EditorData->MoveLayerDown(SelectedLayer);
				})),
			NAME_None,
			EUserInterfaceActionType::Button);
		}
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("LayerOperation", LOCTEXT("LayerOperationLabel", "Layer Operations"));
	{
		// TODO: to be implemented
		//MenuBuilder.AddMenuEntry(
		//	LOCTEXT("RenameLayerLabel", "Rename Layer"),
		//	LOCTEXT("RenameLayerTooltip", "Renames the selected layer."),
		//	FSlateIcon(),
		//	FUIAction(FExecuteAction::CreateLambda([EditorData, SelectedLayers]()
		//		{

		//		})),
		//		NAME_None,
		//		EUserInterfaceActionType::Button);
	
		if (!bIsBaseLayer)
		{
			if (SelectedLayer->GetLayerState() == EUAFLayerState::Disabled)
			{
				MenuBuilder.AddMenuEntry(
				LOCTEXT("EnableLayerLabel", "Enable Layer"),
				LOCTEXT("EnableLayerTooltip", "Enables the selected layer."),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([EditorData, SelectedLayer]()
					{
						EditorData->SetLayerState(SelectedLayer, EUAFLayerState::Enabled);
					})),
				NAME_None,
				EUserInterfaceActionType::Button);
			}
			else
			{
				MenuBuilder.AddMenuEntry(
				LOCTEXT("DisableLayerLabel", "Disable Layer"),
				LOCTEXT("DisableLayerTooltip", "Disables the selected layer."),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([EditorData, SelectedLayer]()
					{
						EditorData->SetLayerState(SelectedLayer, EUAFLayerState::Disabled);
					})),
				NAME_None,
				EUserInterfaceActionType::Button);
			}
		
			MenuBuilder.AddMenuEntry(
				LOCTEXT("DeleteLayerLabel", "Remove Layer"),
				LOCTEXT("DeleteLayerTooltip", "Removes the selected layer."),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([EditorData, SelectedLayer]()
					{
						EditorData->RemoveLayer(SelectedLayer);
					})),
				NAME_None,
				EUserInterfaceActionType::Button);
		
		}
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SLayerStack::OnLayerLayoutChanged()
{
	CacheLayersForList();
	LayerListView->RebuildList();
}

void SLayerStack::OnLayerSelectionChanged(const TWeakObjectPtr<UUAFLayer> SelectedLayer) const
{
	if (SelectedLayer.IsValid())
	{
		LayerListView->SetSelection(SelectedLayer);
	}
	else
	{
		LayerListView->ClearSelection();
	}
}

void SLayerStack::CacheLayersForList()
{
	if (!LayerStack.IsValid())
	{
		CachedLayerPointers.Empty();
		return;
	}
		
	UUAFLayerStack_EditorData* EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData<UUAFLayerStack_EditorData>(LayerStack.Get());
	if (!EditorData)
	{
		CachedLayerPointers.Empty();
		return;
	}
		
	const TArray<TObjectPtr<UUAFLayer>> Layers = EditorData->GetAllLayers();
	CachedLayerPointers.Reset(Layers.Num());
	for (const TObjectPtr<UUAFLayer>& Layer : Layers)
	{
		CachedLayerPointers.Add(Layer);
	}
}
	
}
#undef LOCTEXT_NAMESPACE
	
