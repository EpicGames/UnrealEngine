// Copyright Epic Games, Inc. All Rights Reserved.


#include "Workspace/UAFLayerStackWorkspaceAssetUserData.h"

#include "UAFLayerStack.h"
#include "UAFLayerStack_EditorData.h"
#include "UncookedOnlyUtils.h"
#include "Layers/UAFLayer.h"
#include "Workspace/UAFLayerStackWorkspaceOutlinerData.h"

void UUAFLayerStackWorkspaceAssetUserData::GetRootAssetExport(FAssetRegistryTagsContext Context) const
{
	if (UUAFLayerStack* LayerStack = CastChecked<UUAFLayerStack>(GetOuter()))
	{
		const FName LayerStackIdentifier = LayerStack->GetFName();
		FWorkspaceOutlinerItemExport& RootAssetExport = CachedExports.Exports.Add_GetRef(FWorkspaceOutlinerItemExport(LayerStackIdentifier, LayerStack));
		RootAssetExport.GetData().InitializeAsScriptStruct(FUAFLayerStackWorkspaceOutlinerData::StaticStruct());
		RootAssetExport.GetData().GetMutable<FUAFLayerStackWorkspaceOutlinerData>().SoftAssetPtr = LayerStack;
	}
}

void UUAFLayerStackWorkspaceAssetUserData::GetWorkspaceAssetExports(FAssetRegistryTagsContext Context) const
{
	Super::GetWorkspaceAssetExports(Context);
	
	if (UUAFLayerStack* LayerStack = CastChecked<UUAFLayerStack>(GetOuter()))
	{
		FWorkspaceOutlinerItemExport RootAssetExport = CachedExports.Exports.Num() > 0 ? CachedExports.Exports[0] : FWorkspaceOutlinerItemExport();
		
		// Add each layer as a workspace export 
		UUAFLayerStack_EditorData* EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData<UUAFLayerStack_EditorData>(LayerStack);
		if (EditorData)
		{
			const TArray<TObjectPtr<UUAFLayer>> Layers = EditorData->GetAllLayers();
			for (const TObjectPtr<UUAFLayer>& Layer : Layers)
			{
				if (Layer == nullptr)
				{
					continue;
				}
				
				// Add layer to the export
				const FSoftObjectPath LayerSoftPath = Layer;
				FWorkspaceOutlinerItemExport& LayerExport = CachedExports.Exports.Add_GetRef(FWorkspaceOutlinerItemExport(*LayerSoftPath.GetSubPathUtf8String(), RootAssetExport));
				LayerExport.GetData().InitializeAsScriptStruct(FUAFLayerStackLayerOutlinerData::StaticStruct());
				
				FUAFLayerStackLayerOutlinerData& AssetData = LayerExport.GetData().GetMutable<FUAFLayerStackLayerOutlinerData>();
				AssetData.LayerName = Layer->GetLayerName();
				
				// Add UObjects referenced by layers to the export 
				TArray<const UObject*> ReferencedObjects;
				Layer->GetObjectReferences(ReferencedObjects);
				for (const UObject* Object : ReferencedObjects)
				{
					FSoftObjectPath SoftObjectPath = Object;
					
					FWorkspaceOutlinerItemExport& AssetReference = CachedExports.Exports.Add_GetRef(FWorkspaceOutlinerItemExport(FName(SoftObjectPath.ToString()), LayerExport));
					
					AssetReference.GetData().InitializeAsScriptStruct(FWorkspaceOutlinerAssetReferenceItemData::StaticStruct());
					FWorkspaceOutlinerAssetReferenceItemData& ReferenceData = AssetReference.GetData().GetMutable<FWorkspaceOutlinerAssetReferenceItemData>();
					ReferenceData.ReferredObjectPath = SoftObjectPath;
				}
			}
		}
	}
}
