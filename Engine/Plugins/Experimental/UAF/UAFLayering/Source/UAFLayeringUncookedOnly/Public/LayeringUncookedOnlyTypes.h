// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMCompiler/RigVMCompiler.h"
#include "LayeringUncookedOnlyTypes.generated.h"

class UAnimNextController;
class UUAFLayer;
class UUAFLayerStack;

UENUM(BlueprintType)
enum class EUAFLayerBlendMode : uint8
{
	Blend		UMETA(DisplayName= "Blend", ToolTip = "Behaves like a regular blend and will blend together based on layer weight. Weight 0 == Full Source Value, 1 == Full Layer Value"),
	Additive	UMETA(DisplayName = "Additive", ToolTip = "The layer will get applied as a additive with the weight of the layer"),
	CacheOnly	UMETA(DisplayName = "Cache Only", ToolTip = "The layer will not get applied at all, but will produce a cached output pose if bound"),
	MAX			UMETA(Hidden)
};

UENUM(BlueprintType)
enum class EUAFLayerState : uint8
{
	Enabled			UMETA(DisplayName="Enabled", ToolTip = "This layer is enabled during preview and runtime."),
	PreviewDisabled	UMETA(DisplayName="Preview Disabled", ToolTip = "This layer is disabled during preview but will execute during runtime."),
	Disabled		UMETA(DisplayName="Disabled", ToolTip = "This layer is disabled during preview and runtime.")
};

namespace UE::UAF::Layering
{
	struct FLayerCreationContext
	{
		FLayerCreationContext(const FRigVMCompileSettings& InCompileSettings)
			: CompileSettings(InCompileSettings)
		{
			LayerInputs.SetNum(2);
		}
	
		// The compile setting, can be used to report errors 
		const FRigVMCompileSettings& CompileSettings;
	
		// The stack this layer creation takes place in 
		TObjectPtr<UUAFLayerStack> LayerStack = nullptr;
	
		// The layer currently being generated 
		TObjectPtr<UUAFLayer> Layer = nullptr;
	
		// The UAF Controller controlling the underlying graph of this layer stack
		TObjectPtr<UAnimNextController> GraphController = nullptr;
	
		// Input Pins 
		// 0 == Last layers output 
		// 1 == This layers content output 
		TArray<URigVMPin*> LayerInputs;

		// The location of the last created node
		// Used to build a readable graph under the hood with correct node placement 
		FVector2D LastNodeLocation = FVector2D::ZeroVector;
	};
}
