// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using UnrealBuildTool;
using UnrealBuildTool.Rules;

public class UAFLayeringUncookedOnly : ModuleRules
{
	public UAFLayeringUncookedOnly(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
				"WorkspaceEditor",
				"UAF",
				"UAFUncookedOnly",
				"UAFAnimGraphUncookedOnly",
				"PropertyEditor",
				"HierarchyTableRuntime",
				"HierarchyTableAnimationRuntime",
				"InputCore"
				
				// ... add other public dependencies that you statically link with here ...
			}
			);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"UAFLayering",
				"RigVM",
				"RigVMDeveloper",
				"UAFAnimGraph",
				"UnrealEd",
				"EditorWidgets",
				"EditorInteractiveToolsFramework",
				"InteractiveToolsFramework",
				// ... add private dependencies that you statically link with here ...	
			}
		);
		
		if (Target.bBuildEditor)
		{
			PrivateIncludePathModuleNames.AddRange(
				new string[]
				{
					"UAFLayeringEditor",
				}
			);
			
		}
		
		bAllowUETypesInNamespaces = true;
	}
}
