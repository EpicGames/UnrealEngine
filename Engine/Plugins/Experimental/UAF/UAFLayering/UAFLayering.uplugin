{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "UAF Layering",
	"Description": "Framework to define a layering setup in UAF",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Plugins": 
	[	
		{
			"Name": "Workspace",
			"Enabled": true
		},
		{
			"Name": "UAF",
			"Enabled": true
		},
		{
			"Name": "UAFAnimGraph",
			"Enabled": true
		}
	],
	"Modules":
	[
		{
			"Name": "UAFLayering",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "UAFLayeringEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "UAFLayeringUncookedOnly",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		}
	]
}